{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Engine Asset Definitions",
	"Description": "",
	"Category": "Other",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault" : true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "EngineAssetDefinitions",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}