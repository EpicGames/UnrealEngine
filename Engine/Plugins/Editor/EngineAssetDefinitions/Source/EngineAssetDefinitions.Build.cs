// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EngineAssetDefinitions : ModuleRules
{
	public EngineAssetDefinitions(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[]
			{
				"SkeletalMeshEditor",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"ContentBrowser",
				"ContentBrowserData",
				"JsonObjectGraph",
				"AssetDefinition",
				"UnrealEd",
				"ToolMenus",
				"CoreUObject",
				"RHI",
				"Engine",
				"AssetRegistry",
				"AssetTools",
				"SlateCore",
				"InputCore",
				"Slate",
				"Kismet",
				"DesktopPlatform",
				"Foliage",
				"Landscape",
				"MaterialEditor",
				"PhysicsCore",
				"Kismet",
				"GameProjectGeneration",
				"AnimationBlueprintEditor",
				"ToolWidgets",
                "SourceControl",
				"Blutility",
				"UMGEditor",
				"ToolWidgets",
				"BlueprintGraph"
			}
		);
		
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"VirtualTexturingEditor",
				"CurveAssetEditor",
				"StaticMeshEditor",
				"TextureEditor",
				"MaterialEditor",
				"PhysicsAssetEditor",
				"FontEditor",
				"DataTableEditor",
				"CurveTableEditor",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"VirtualTexturingEditor",
				"CurveAssetEditor",
				"StaticMeshEditor",
				"TextureEditor",
				"PhysicsAssetEditor",
				"FontEditor",
				"DataTableEditor",
				"CurveTableEditor",
			}
		);
	}
}
