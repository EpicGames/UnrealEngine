// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AssetDefinition_BlendSpace.h"

#include "AssetDefinition_AimOffset.generated.h"

class UAssetDefinition_BlendSpace;

UCLASS()
class UAssetDefinition_AimOffset : public UAssetDefinition_BlendSpace
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AimOffset", "Aim Offset"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 162, 232)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAimOffsetBlendSpace::StaticClass(); }
	// UAssetDefinition End
};
