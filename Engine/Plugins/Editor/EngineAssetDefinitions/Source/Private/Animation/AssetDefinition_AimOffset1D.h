// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AimOffsetBlendSpace1D.h"
#include "Animation/AssetDefinition_BlendSpace1D.h"

#include "AssetDefinition_AimOffset1D.generated.h"

class UAssetDefinition_BlendSpace1D;

UCLASS()
class UAssetDefinition_AimOffset1D : public UAssetDefinition_BlendSpace1D
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AimOffset1D", "Aim Offset 1D"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 162, 232)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAimOffsetBlendSpace1D::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Animation / NSLOCTEXT("AssetTypeActions", "AnimLegacySubMenu", "Legacy") };
		return Categories;
	}
	// UAssetDefinition End
};
