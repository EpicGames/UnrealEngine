// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AssetDefinition_AnimBank.h"

#include "ContentBrowserMenuContexts.h"
#include "Misc/PackageName.h"
#include "ToolMenus.h"
#include "Misc/MessageDialog.h"
#include "ContentBrowserModule.h"
#include "Framework/Application/SlateApplication.h"
#include "SBlueprintDiff.h"
#include "SSkeletonWidget.h"
#include "Styling/SlateIconFinder.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IAnimationBlueprintEditorModule.h"
#include "IAssetTools.h"
#include "Algo/AllOf.h"
#include "Algo/NoneOf.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "SPrimaryButton.h"
#include "SSimpleButton.h"
#include "BlueprintEditorSettings.h"
#include "Logging/MessageLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_AnimBank)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

TSharedPtr<SWidget> UAssetDefinition_AnimBank::GetThumbnailOverlay(const FAssetData& AssetData) const
{
	const FSlateBrush* Icon = FSlateIconFinder::FindIconBrushForClass(UAnimBank::StaticClass());

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetNoBrush())
		.Visibility(EVisibility::HitTestInvisible)
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 3.0f))
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
	[
		SNew(SImage)
		.Image(Icon)
	];
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_AnimBank
{

}

#undef LOCTEXT_NAMESPACE
