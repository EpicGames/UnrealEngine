// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimBank.h"
#include "Animation/AssetDefinition_AnimationAsset.h"
#include "Animation/AssetDefinition_TransformProviderData.h"
#include "AssetDefinition_AnimBank.generated.h"

class UAssetDefinition_AnimationAsset;

UCLASS()
class UAssetDefinition_AnimBank : public UAssetDefinition_AnimationAsset
{
	GENERATED_BODY()

public:
	virtual FText GetAssetDisplayName() const override
	{
		return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimBank", "Animation Bank");
	}
	
	virtual TSoftClassPtr<UObject> GetAssetClass() const override
	{
		return UAnimBank::StaticClass();
	}

	virtual FLinearColor GetAssetColor() const override
	{
		return FLinearColor(FColor(237, 100, 36));
	}

	virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override;
};

UCLASS()
class UAssetDefinition_AnimBankData : public UAssetDefinition_TransformProviderData
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimBankData", "Animation Bank Data"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimBankData::StaticClass(); }
	// UAssetDefinition End
};

