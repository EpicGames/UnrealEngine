// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AssetDefinition_AnimBlueprint.h"

#include "ContentBrowserMenuContexts.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/PackageName.h"
#include "ToolMenus.h"
#include "Misc/MessageDialog.h"
#include "Factories/AnimBlueprintFactory.h"
#include "ContentBrowserModule.h"
#include "Framework/Application/SlateApplication.h"
#include "SBlueprintDiff.h"
#include "SSkeletonWidget.h"
#include "Styling/SlateIconFinder.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IAnimationBlueprintEditorModule.h"
#include "IAssetTools.h"
#include "Algo/AllOf.h"
#include "Algo/NoneOf.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "SPrimaryButton.h"
#include "SSimpleButton.h"
#include "BlueprintEditorSettings.h"
#include "Logging/MessageLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_AnimBlueprint)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

namespace UE::AnimBlueprint
{
	bool ReplaceMissingSkeleton(const TArray<TObjectPtr<UObject>>& InAnimationAssets)
	{
		// record anim assets that need skeleton replaced
		const TArray<TWeakObjectPtr<UObject>> AnimsToFix = TArray<TWeakObjectPtr<UObject>>(InAnimationAssets);
		
		// get a skeleton from the user and replace it
		const TSharedPtr<SReplaceMissingSkeletonDialog> PickSkeletonWindow = SNew(SReplaceMissingSkeletonDialog).AnimAssets(AnimsToFix);
		const bool bWasSkeletonReplaced = PickSkeletonWindow.Get()->ShowModal();
		return bWasSkeletonReplaced;
	}

	EBlueprintType GetBlueprintType(const FAssetData& InAssetData)
	{
		const UEnum* BlueprintTypeEnum = StaticEnum<EBlueprintType>();
		const FString EnumString = InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintType));
		const EBlueprintType BlueprintType = (!EnumString.IsEmpty() ? static_cast<EBlueprintType>(BlueprintTypeEnum->GetValueByName(*EnumString)) : BPTYPE_Normal);
		return BlueprintType;
	}

	bool IsAnimBlueprintTemplate(const FAssetData& InAssetData)
	{
		return InAssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UAnimBlueprint, bIsTemplate));
	}

	TSoftObjectPtr<USkeleton> GetAnimBlueprintTargetSkeleton(const FAssetData& InAssetData)
	{
		return TSoftObjectPtr<USkeleton>(FSoftObjectPath(InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UAnimBlueprint, TargetSkeleton))));
	}
}

FText UAssetDefinition_AnimBlueprint::GetAssetDisplayName(const FAssetData& AssetData) const
{
	FString OutBlueprintType;
	if (AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintType), OutBlueprintType))
	{
		if (OutBlueprintType == TEXT("BPTYPE_Interface"))
		{
			return LOCTEXT("AssetTypeActions_AnimLayerInterface", "Animation Layer Interface");
		}
	}

	return Super::GetAssetDisplayName(AssetData);
}

void UAssetDefinition_AnimBlueprint::BuildFilters(TArray<FAssetFilterData>& OutFilters) const
{
	// Intentionally skipping UAssetDefinition_Blueprint.
	UAssetDefinition::BuildFilters(OutFilters);

	{
		FAssetFilterData Filter;
		Filter.Name = GetAssetClass().ToSoftObjectPath().ToString() + TEXT("Interface");
		Filter.DisplayText = LOCTEXT("AssetTypeActions_AnimLayerInterface", "Animation Layer Interface");
		Filter.FilterCategories = { EAssetCategoryPaths::Animation };
		Filter.Filter.ClassPaths.Add(GetAssetClass().ToSoftObjectPath().GetAssetPath());
		Filter.Filter.TagsAndValues.Add(GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintType), FString(TEXT("BPTYPE_Interface")));
		OutFilters.Add(MoveTemp(Filter));
	}
}

UFactory* UAssetDefinition_AnimBlueprint::GetFactoryForBlueprintType(UBlueprint* InBlueprint) const
{
	UAnimBlueprint* AnimBlueprint = CastChecked<UAnimBlueprint>(InBlueprint);

	if (InBlueprint->BlueprintType == BPTYPE_Interface)
	{
		return NewObject<UAnimLayerInterfaceFactory>();
	}
	else
	{
		UAnimBlueprintFactory* AnimBlueprintFactory = NewObject<UAnimBlueprintFactory>();
		AnimBlueprintFactory->ParentClass = TSubclassOf<UAnimInstance>(*InBlueprint->GeneratedClass);
		AnimBlueprintFactory->TargetSkeleton = AnimBlueprint->TargetSkeleton;
		AnimBlueprintFactory->bTemplate = AnimBlueprint->bIsTemplate;
		return AnimBlueprintFactory;
	}
}

EAssetCommandResult UAssetDefinition_AnimBlueprint::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UBlueprint* OldBlueprint = Cast<UBlueprint>(DiffArgs.OldAsset);
	const UBlueprint* NewBlueprint = Cast<UBlueprint>(DiffArgs.NewAsset);

	if (NewBlueprint == nullptr && OldBlueprint == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}
	
	SBlueprintDiff::CreateDiffWindow(OldBlueprint, NewBlueprint, DiffArgs.OldRevision, DiffArgs.NewRevision, UAnimBlueprint::StaticClass());
	return EAssetCommandResult::Handled;
}


EAssetCommandResult UAssetDefinition_AnimBlueprint::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<FAssetData> OutAssetsThatFailedToLoad;
	for (UAnimBlueprint* AnimBlueprint : OpenArgs.LoadObjects<UAnimBlueprint>({}, &OutAssetsThatFailedToLoad))
	{
		if (AnimBlueprint->SkeletonGeneratedClass && AnimBlueprint->GeneratedClass)
		{
			if (AnimBlueprint->BlueprintType != BPTYPE_Interface && !AnimBlueprint->TargetSkeleton && !AnimBlueprint->bIsTemplate)
			{
				// For derived anim blueprints, the skeleton is determined by the root anim BP. If we don't have a skeleton it shouldn't either.
				UAnimBlueprint* RootAnimBlueprint = UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint);

				FText ShouldRetargetMessage = LOCTEXT("ShouldRetarget_Message", "Could not find the skeleton for Anim Blueprint '{BlueprintName}' Would you like to choose a new one?");

				if (RootAnimBlueprint && RootAnimBlueprint->TargetSkeleton)
				{
					// If our base has a new skeleton, but we haven't updated to it yet. Ask for confirmation to update as we may reference assets exclusive to the old skeleton (sequences, etc).
					ShouldRetargetMessage = LOCTEXT("ShouldRetargetToParent_Message", "Anim Blueprint '{BlueprintName}' does not have a skeleton. But parent '{RootAnimBlueprint}' does. Would you like to update to the parent skeleton?");
				}
				else if (RootAnimBlueprint)
				{
					// If our base does not have a skeleton, offer to update the base
					ShouldRetargetMessage = LOCTEXT("ShouldRetargetParent_Message", "Could not find the skeleton for Anim Blueprint '{BlueprintName}' in parent '{RootAnimBlueprint}' Would you like to choose a new parent skeleton?");
				}
				
				FFormatNamedArguments Arguments;
				Arguments.Add( TEXT("BlueprintName"), FText::FromString(AnimBlueprint->GetName()));
				if (RootAnimBlueprint)
				{
					Arguments.Add(TEXT("RootAnimBlueprint"), FText::FromString(RootAnimBlueprint->GetName()));
				}

				if (FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(ShouldRetargetMessage, Arguments)) == EAppReturnType::Yes)
				{
					bool bSkeletonReplaced = false;
					if (RootAnimBlueprint && RootAnimBlueprint->TargetSkeleton)
					{
						// Update to parent skeleton
						constexpr bool bRetargetReferredAssets = true;
						constexpr bool bConvertSpaces = false;

						FAnimationRetargetContext RetargetContext({ TWeakObjectPtr<UObject>(AnimBlueprint) }, bRetargetReferredAssets, bConvertSpaces);
						const FNameDuplicationRule* NameRule = nullptr;

						EditorAnimUtils::RetargetAnimations(nullptr, RootAnimBlueprint->TargetSkeleton, RetargetContext, bRetargetReferredAssets, NameRule);
						bSkeletonReplaced = true;
					}
					else
					{
						TArray<TObjectPtr<UObject>> AssetsToRetarget;

						if (RootAnimBlueprint && !RootAnimBlueprint->TargetSkeleton)
						{
							AssetsToRetarget.Add(RootAnimBlueprint);
						}
						AssetsToRetarget.Add(AnimBlueprint);

						// Spawn skeleton selector modal
						bSkeletonReplaced = UE::AnimBlueprint::ReplaceMissingSkeleton(AssetsToRetarget);
					}

					if (!bSkeletonReplaced)
					{
						return EAssetCommandResult::Handled; // Persona will crash if trying to load asset without a skeleton
					}
				}
				else
				{
					return EAssetCommandResult::Handled;
				}
			}
			
			const bool bBringToFrontIfOpen = true;
#if WITH_EDITOR
			if (IAssetEditorInstance* EditorInstance = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, bBringToFrontIfOpen))
			{
				EditorInstance->FocusWindow(AnimBlueprint);
			}
			else
#endif
			{
				IAnimationBlueprintEditorModule& AnimationBlueprintEditorModule = FModuleManager::LoadModuleChecked<IAnimationBlueprintEditorModule>("AnimationBlueprintEditor");
				AnimationBlueprintEditorModule.CreateAnimationBlueprintEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, AnimBlueprint);
			}
		}
		else
		{
			FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("FailedToLoadCorruptAnimBlueprint", "The Anim Blueprint could not be loaded because it is corrupt."));
		}
	}

	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_AnimBlueprint
{
	void ExecuteFindSkeleton(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		TArray<UObject*> ObjectsToSync;
		for (UAnimBlueprint* AnimBlueprint : CBContext->LoadSelectedObjects<UAnimBlueprint>())
		{
			if (USkeleton* Skeleton = AnimBlueprint->TargetSkeleton)
			{
				ObjectsToSync.AddUnique(Skeleton);
			}
		}
    
		if ( ObjectsToSync.Num() > 0 )
		{
			IAssetTools::Get().SyncBrowserToAssets(ObjectsToSync);
		}
	}

	void CompileAnimBlueprintWithLog(UAnimBlueprint* AnimBlueprint)
	{
		FCompilerResultsLog	ResultsLog;
		ResultsLog.SetSourcePath(AnimBlueprint->GetPathName());
		ResultsLog.BeginEvent(TEXT("Compile"));

		FKismetEditorUtilities::CompileBlueprint(AnimBlueprint, EBlueprintCompileOptions::None, &ResultsLog);
		ResultsLog.EndEvent();
		FMessageLog BlueprintLogger("AnimBlueprintLog");
		BlueprintLogger.NewPage(FText::Format(LOCTEXT("CompileAnimBPPageLabel", "{0}"), FText::FromString(AnimBlueprint->GetName())));
		BlueprintLogger.AddMessages(ResultsLog.Messages);
		if ((ResultsLog.NumErrors) > 0)
		{
			BlueprintLogger.Notify(FText::Format(LOCTEXT("CompileAnimBPErrors", "AnimBP compilation generated {0} error(s)"), ResultsLog.NumErrors));
		}
		else if (ResultsLog.NumWarnings)
		{
			BlueprintLogger.Notify(FText::Format(LOCTEXT("CompileAnimBPWarnings", "AnimBP compilation generated {0} warnings(s)"), ResultsLog.NumWarnings));
		}
	}

	void ExecuteAssignSkeleton(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		TSharedRef<SWindow> WidgetWindow = SNew(SWindow)
			.Title(LOCTEXT("ChooseSkeletonWindowTitle", "Choose Skeleton"))
			.ClientSize(FVector2D(400, 600));

		TSharedPtr<SSkeletonSelectorWindow> SkeletonSelectorWindow;
		WidgetWindow->SetContent
		(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(SkeletonSelectorWindow, SSkeletonSelectorWindow)
				.WidgetWindow(WidgetWindow)
			]
		);

		GEditor->EditorAddModalWindow(WidgetWindow);

		if (USkeleton* SelectedSkeleton = SkeletonSelectorWindow->GetSelectedSkeleton())
		{
			for (UAnimBlueprint* AnimBlueprint : CBContext->LoadSelectedObjects<UAnimBlueprint>())
			{
				if (AnimBlueprint->TargetSkeleton != SelectedSkeleton)
				{
					UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
					const int32 NumEditorsClosed = AssetEditorSubsystem->CloseAllEditorsForAsset(AnimBlueprint);

					AnimBlueprint->Modify();
					AnimBlueprint->TargetSkeleton = SelectedSkeleton;
					AnimBlueprint->PostEditChange();

					CompileAnimBlueprintWithLog(AnimBlueprint);

					if (NumEditorsClosed > 0)
					{
						AssetEditorSubsystem->OpenEditorForAsset(AnimBlueprint);
					}
				}
			}
		}
	}

	void ExecuteCreateTemplate(const FToolMenuContext& InContext, const FAssetData InAnimBlueprintAsset)
	{
		if (UAnimBlueprint* SourceBP = Cast<UAnimBlueprint>(InAnimBlueprintAsset.GetAsset()))
		{
			UClass* TargetParentClass = SourceBP->GeneratedClass;

			if (!FKismetEditorUtilities::CanCreateBlueprintOfClass(TargetParentClass))
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("InvalidClassToMakeBlueprintFrom", "Invalid class with which to make a Blueprint."));
				return;
			}

			FString TargetName;
			FString TargetPackageName;
			IAssetTools::Get().CreateUniqueAssetName(SourceBP->GetOutermost()->GetName(), TEXT("_Template"), TargetPackageName, TargetName);
			if (UAnimBlueprint* NewAnimBP = Cast<UAnimBlueprint>(IAssetTools::Get().DuplicateAssetWithDialog(TargetName, TargetPackageName, SourceBP)))
			{
				NewAnimBP->Modify();
				NewAnimBP->bIsTemplate = true;
				NewAnimBP->TargetSkeleton = nullptr;
				NewAnimBP->PostEditChange();

				CompileAnimBlueprintWithLog(NewAnimBP);

				UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				AssetEditorSubsystem->OpenEditorForAsset(NewAnimBP);
			}
		}
	}

	void ExecuteNewSkeletonChildBlueprint(const FToolMenuContext& InContext, const FAssetData InAnimBlueprintAsset)
	{
		TArray<FSoftObjectPath> CompatibleSkeletonPaths;
		TSoftObjectPtr<USkeleton> AnimBlueprintSkeletonPtr = UE::AnimBlueprint::GetAnimBlueprintTargetSkeleton(InAnimBlueprintAsset);
		if (USkeleton* AnimBlueprintSkeleton = AnimBlueprintSkeletonPtr.LoadSynchronous())
		{
			TArray<FAssetData> CompatibleSkeletonAssets;
			AnimBlueprintSkeleton->GetCompatibleSkeletonAssets(CompatibleSkeletonAssets);

			Algo::Transform(CompatibleSkeletonAssets, CompatibleSkeletonPaths, [](const FAssetData& InAsset) { return InAsset.GetSoftObjectPath(); });
		}

		FAssetData SelectedSkeletonAsset;
		auto HandleAssetSelected = [&SelectedSkeletonAsset](const FAssetData& InSelectedSkeletonAsset)
		{
			SelectedSkeletonAsset = InSelectedSkeletonAsset;
		};
		
		FAssetPickerConfig AssetPickerConfig;
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateLambda([HandleAssetSelected](const TArray<FAssetData>& SelectedAssetData)
		{
			if (SelectedAssetData.Num() == 1)
			{
				HandleAssetSelected(SelectedAssetData[0]);
			}
		});
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda(HandleAssetSelected);
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.Filter.SoftObjectPaths = MoveTemp(CompatibleSkeletonPaths);
		AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		bool bClickedOk = false;
		TSharedPtr<SWindow> Dialog;
		Dialog = SNew(SWindow)
				.Title(FText(LOCTEXT("CompatibleSkeletonDialog", "Compatible Skeleton for Animation Blueprint")))
				.SizingRule(ESizingRule::Autosized);

		Dialog->SetContent(
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			[
				SNew(SBox)
				.WidthOverride(300.0f)
				.HeightOverride(400.0f)
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.0f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SPrimaryButton)
					.OnClicked_Lambda([&bClickedOk, WeakDialog = TWeakPtr<SWindow>(Dialog)]()
					{
						bClickedOk = true;
						if(TSharedPtr<SWindow> PinnedDialog = WeakDialog.Pin())
						{
							PinnedDialog->RequestDestroyWindow();
						}
						
						return FReply::Handled();
					})
					.IsEnabled_Lambda([&SelectedSkeletonAsset]()
					{
						return SelectedSkeletonAsset.IsValid();
					})
					.Text(LOCTEXT("CreateChildButton", "Create Child"))
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([WeakDialog = TWeakPtr<SWindow>(Dialog)]()
					{
						if(TSharedPtr<SWindow> PinnedDialog = WeakDialog.Pin())
						{
							PinnedDialog->RequestDestroyWindow();
						}

						return FReply::Handled();
					})
					.Text(LOCTEXT("CancelButton", "Cancel"))
				]
			]);

		FSlateApplication::Get().AddModalWindow(Dialog.ToSharedRef(), FGlobalTabmanager::Get()->GetRootWindow());

		if(bClickedOk && SelectedSkeletonAsset.IsValid())	// Clicked OK
		{
			if (UAnimBlueprint* TargetParentBP = Cast<UAnimBlueprint>(InAnimBlueprintAsset.GetAsset()))
			{
				USkeleton* TargetSkeleton = CastChecked<USkeleton>(SelectedSkeletonAsset.GetAsset());
				UClass* TargetParentClass = TargetParentBP->GeneratedClass;

				if (!FKismetEditorUtilities::CanCreateBlueprintOfClass(TargetParentClass))
				{
					FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("InvalidClassToMakeBlueprintFrom", "Invalid class with which to make a Blueprint."));
					return;
				}

				FString Name;
				FString PackageName;
				IAssetTools::Get().CreateUniqueAssetName(TargetParentBP->GetOutermost()->GetName(), TEXT("_Child"), PackageName, Name);
				const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

				UAnimBlueprintFactory* AnimBlueprintFactory = NewObject<UAnimBlueprintFactory>();
				AnimBlueprintFactory->ParentClass = TSubclassOf<UAnimInstance>(*TargetParentBP->GeneratedClass);
				AnimBlueprintFactory->TargetSkeleton = TargetSkeleton;
				AnimBlueprintFactory->bTemplate = false;

				ContentBrowserModule.Get().CreateNewAsset(Name, PackagePath, TargetParentBP->GetClass(), AnimBlueprintFactory);
			}
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UAnimBlueprint::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					if (const FAssetData* SelectedAnimBlueprintPtr = CBContext->GetSingleSelectedAssetOfType(UAnimBlueprint::StaticClass(), EIncludeSubclasses::No))
					{
						// Accept (non-interface) template anim BPs or anim BPs with compatible skeletons
						if (SelectedAnimBlueprintPtr &&
							UE::AnimBlueprint::GetBlueprintType(*SelectedAnimBlueprintPtr) != BPTYPE_Interface &&
							(
								(UE::AnimBlueprint::GetAnimBlueprintTargetSkeleton(*SelectedAnimBlueprintPtr).IsNull() && UE::AnimBlueprint::IsAnimBlueprintTemplate(*SelectedAnimBlueprintPtr))
								||
								(!UE::AnimBlueprint::GetAnimBlueprintTargetSkeleton(*SelectedAnimBlueprintPtr).IsNull())
							)
						)
						{
							const TAttribute<FText> Label = LOCTEXT("AnimBlueprint_NewSkeletonChildBlueprint", "Create Child Anim Blueprint with Skeleton");
							const TAttribute<FText> ToolTip = LOCTEXT("AnimBlueprint_NewSkeletonChildBlueprint_Tooltip", "Create a child Anim Blueprint that uses a different compatible skeleton.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Blueprint");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewSkeletonChildBlueprint, *SelectedAnimBlueprintPtr);
							InSection.AddMenuEntry("AnimBlueprint_NewSkeletonChildBlueprint", Label, ToolTip, Icon, UIAction);
						}
					}
					
					if (Algo::NoneOf(CBContext->SelectedAssets, &UE::AnimBlueprint::IsAnimBlueprintTemplate) &&
						Algo::AllOf(CBContext->SelectedAssets, [](const auto& AssetData) { return BPTYPE_Normal == UE::AnimBlueprint::GetBlueprintType(AssetData); } ))
					{
						{
							const TAttribute<FText> Label = LOCTEXT("AnimBlueprint_FindSkeleton", "Find Skeleton");
							const TAttribute<FText> ToolTip = LOCTEXT("AnimBlueprint_FindSkeletonTooltip", "Finds the skeleton used by the selected Anim Blueprints in the content browser.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Find");
				
							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindSkeleton);
							InSection.AddMenuEntry("AnimBlueprint_FindSkeleton", Label, ToolTip, Icon, UIAction);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("AnimBlueprint_AssignSkeleton", "Assign Skeleton");
							const TAttribute<FText> ToolTip = LOCTEXT("AnimBlueprint_AssignSkeletonTooltip", "Assigns a skeleton to the selected Animation Blueprint(s).");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.AssignSkeleton");
				
							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteAssignSkeleton);
							InSection.AddMenuEntry("AnimBlueprint_AssignSkeleton", Label, ToolTip, Icon, UIAction);
						}

						{
							if (const FAssetData* SelectedAnimBlueprintPtr = CBContext->GetSingleSelectedAssetOfType(UAnimBlueprint::StaticClass(), EIncludeSubclasses::No))
							{
								// Accept (non-interface) non templated anim BPs or anim BPs
								if (SelectedAnimBlueprintPtr
									&& UE::AnimBlueprint::GetBlueprintType(*SelectedAnimBlueprintPtr) != BPTYPE_Interface
									&& !UE::AnimBlueprint::IsAnimBlueprintTemplate(*SelectedAnimBlueprintPtr))
								{
									const TAttribute<FText> Label = LOCTEXT("AnimBlueprint_CreateTemplate", "Create Template Animation Blueprint");
									const TAttribute<FText> ToolTip = LOCTEXT("AnimBlueprint_CreateTemplateTooltip", "Creates AnimBP Template(s) from the selected Animation Blueprint(s).");
									const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.CreateAnimAsset");

									FToolUIAction UIAction;
									UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateTemplate, *SelectedAnimBlueprintPtr);
									InSection.AddMenuEntry("AnimBlueprint_CreateTemplate", Label, ToolTip, Icon, UIAction);
								}
							}
						}
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
