// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Script/AssetDefinition_Blueprint.h"
#include "Animation/AnimBlueprint.h"

#include "AssetDefinition_AnimBlueprint.generated.h"

struct FToolMenuContext;
struct FAssetData;
class UFactory;

UCLASS()
class UAssetDefinition_AnimBlueprint : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()

public:
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimBlueprint", "Animation Blueprint"); }
	virtual FText GetAssetDisplayName(const FAssetData& AssetData) const override;
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(200, 116, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimBlueprint::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Basic, EAssetCategoryPaths::Animation };
		return Categories;
	}
	
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;

	virtual void BuildFilters(TArray<FAssetFilterData>& OutFilters) const override;
	
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End

	// UAssetDefinition_Blueprint Implementation
	virtual UFactory* GetFactoryForBlueprintType(UBlueprint* InBlueprint) const override;
	// UAssetDefinition_Blueprint End
};
