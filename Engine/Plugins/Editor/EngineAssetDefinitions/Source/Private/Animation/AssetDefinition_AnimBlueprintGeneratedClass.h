// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Script/AssetDefinition_BlueprintGeneratedClass.h"

#include "AssetDefinition_AnimBlueprintGeneratedClass.generated.h"

UCLASS()
class UAssetDefinition_AnimBlueprintGeneratedClass : public UAssetDefinition_BlueprintGeneratedClass
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimBlueprintGeneratedClass", "Compiled Anim Blueprint Class"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(240, 156, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimBlueprintGeneratedClass::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Animation };
		return Categories;
	}
	// UAssetDefinition End
};
