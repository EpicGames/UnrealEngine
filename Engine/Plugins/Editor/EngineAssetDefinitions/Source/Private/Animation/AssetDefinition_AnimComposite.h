// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimComposite.h"
#include "Animation/AssetDefinition_AnimationAsset.h"

#include "AssetDefinition_AnimComposite.generated.h"

class UAssetDefinition_AnimationAsset;

UCLASS()
class UAssetDefinition_AnimComposite : public UAssetDefinition_AnimationAsset
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimComposite", "Animation Composite"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(181, 230, 29)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimComposite::StaticClass(); }
	// UAssetDefinition End
};
