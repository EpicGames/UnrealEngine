// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AnimMontage.h"

#include "AnimationEditorUtils.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_AnimMontage)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_AnimMontage
{
	const static FName NAME_HasParentAsset("HasParentAsset");
	static TOptional<bool> HasParentAsset(const FAssetData& AnimMontage)
	{
		bool bHasParentAsset = false;
		if (!AnimMontage.GetTagValue<bool>(NAME_HasParentAsset, bHasParentAsset))
		{
			return TOptional<bool>();
		}
		
		return bHasParentAsset;
	}

	void ExecuteCreateChildAnimMontage(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UAnimMontage*> AnimMontages = Context->LoadSelectedObjects<UAnimMontage>();
		
        // only show child montage if inobjects are not child montage already
        bool bContainsChildMontage = false;
        for (UAnimMontage* AnimMontage : AnimMontages)
        {
        	bContainsChildMontage |= AnimMontage->HasParentAsset();
        	if (bContainsChildMontage)
        	{
        		break;
        	}
        }
		
		if (!bContainsChildMontage && AnimMontages.Num() > 0)
		{
			TArray<UObject*> ObjectsToSync;
			// need to know source and target
			 for (UAnimMontage* ParentMontage : AnimMontages)
             {
				UAnimMontage* NewAsset = AnimationEditorUtils::CreateAnimationAsset<UAnimMontage>(ParentMontage->GetSkeleton(), ParentMontage->GetOutermost()->GetName(), TEXT("_Child"));
				if (NewAsset)
				{
					NewAsset->SetParentAsset(ParentMontage);
					ObjectsToSync.Add(NewAsset);
				}
			}

			IAssetTools::Get().SyncBrowserToAssets(ObjectsToSync);
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UAnimMontage::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection);

				// Only show create child montage option if none of the selected assets are child montages. 
				if (!CBContext->SelectedAssets.ContainsByPredicate([](const FAssetData& InAsset) { return HasParentAsset(InAsset).Get(false); }))
				{
					const TAttribute<FText> Label = LOCTEXT("AnimMontage_CreateChildMontage", "Create Child Montage");
					const TAttribute<FText> ToolTip = LOCTEXT("AnimMontage_CreateChildMontageTooltip", "Create Child Animation Montage and remap to another animation assets.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimMontage");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateChildAnimMontage);
					InSection.AddMenuEntry("AnimMontage_CreateChildMontage", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
