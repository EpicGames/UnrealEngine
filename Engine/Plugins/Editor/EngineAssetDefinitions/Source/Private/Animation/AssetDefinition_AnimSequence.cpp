// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AnimSequence.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "EditorReimportHandler.h"
#include "Animation/AnimMontage.h"
#include "Factories/AnimCompositeFactory.h"
#include "Factories/AnimStreamableFactory.h"
#include "Factories/AnimMontageFactory.h"
#include "Factories/PoseAssetFactory.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimStreamable.h"
#include "Animation/PoseAsset.h"
#include "IAssetTools.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ContentBrowserMenuContexts.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_AnimSequence)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_AnimSequence
{
	/** Delegate used when creating Assets from an AnimSequence */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnConfigureFactory, class UFactory*, class UAnimSequence* );

	int32 GEnableAnimStreamable = 0;
	static const TCHAR* AnimStreamableCVarName = TEXT("a.EnableAnimStreamable");

	static FAutoConsoleVariableRef CVarEnableAnimStreamable(
		AnimStreamableCVarName,
		GEnableAnimStreamable,
		TEXT("1 = Enables ability to make Anim Streamable assets. 0 = off"));
	
	void ExecuteNewAnimComposite(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		IAssetTools::Get().CreateAssetsFrom<UAnimSequence>(
			CBContext->LoadSelectedObjects<UAnimSequence>(), UAnimComposite::StaticClass(), TEXT("_Composite"), [](UAnimSequence* SourceObject)
			{
				UAnimCompositeFactory* Factory = NewObject<UAnimCompositeFactory>();
				Factory->SourceAnimation = SourceObject;
				return Factory;
			}
		);
	}

	void ExecuteNewAnimMontage(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		IAssetTools::Get().CreateAssetsFrom<UAnimSequence>(
			CBContext->LoadSelectedObjects<UAnimSequence>(), UAnimMontage::StaticClass(), TEXT("_Montage"), [](UAnimSequence* SourceObject)
			{
				UAnimMontageFactory* Factory = NewObject<UAnimMontageFactory>();
				Factory->SourceAnimation = SourceObject;
				return Factory;
			}
		);
	}

	void ExecuteNewAnimStreamable(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		IAssetTools::Get().CreateAssetsFrom<UAnimSequence>(
			CBContext->LoadSelectedObjects<UAnimSequence>(), UAnimStreamable::StaticClass(), TEXT("_Streamable"), [](UAnimSequence* SourceObject)
			{
				UAnimStreamableFactory* Factory = NewObject<UAnimStreamableFactory>();
				Factory->SourceAnimation = SourceObject;
				return Factory;
			}
		);
	}

	void ExecuteNewPoseAsset(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		IAssetTools::Get().CreateAssetsFrom<UAnimSequence>(
			CBContext->LoadSelectedObjects<UAnimSequence>(), UPoseAsset::StaticClass(), TEXT("_PoseAsset"), [](UAnimSequence* SourceObject)
			{
				UPoseAssetFactory* Factory = NewObject<UPoseAssetFactory>();
				Factory->SourceAnimation = SourceObject;
				return Factory->ConfigureProperties() ? Factory : nullptr;
			}
		);
	}
	
	void FillCreateMenu(UToolMenu* Menu)
	{
		IAssetTools& AssetTools = IAssetTools::Get();

		FToolMenuSection& Section = Menu->FindOrAddSection("CreateAssetsMenu");

		if (AssetTools.IsAssetClassSupported(UAnimComposite::StaticClass()))
		{
			const TAttribute<FText> Label = LOCTEXT("AnimSequence_NewAnimComposite", "Create AnimComposite");
			const TAttribute<FText> ToolTip = LOCTEXT("AnimSequence_NewAnimCompositeTooltip", "Creates an AnimComposite using the selected anim sequence.");
			const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimComposite");

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewAnimComposite);
			Section.AddMenuEntry("AnimSequence_NewAnimComposite", Label, ToolTip, Icon, UIAction);
		}

		if (AssetTools.IsAssetClassSupported(UAnimMontage::StaticClass()))
		{
			const TAttribute<FText> Label = LOCTEXT("AnimSequence_NewAnimMontage", "Create AnimMontage");
			const TAttribute<FText> ToolTip = LOCTEXT("AnimSequence_NewAnimMontageTooltip", "Creates an AnimMontage using the selected anim sequence.");
			const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimMontage");

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewAnimMontage);
			Section.AddMenuEntry("AnimSequence_NewAnimMontage", Label, ToolTip, Icon, UIAction);
		}

		if (GEnableAnimStreamable == 1)
		{
			if (AssetTools.IsAssetClassSupported(UAnimStreamable::StaticClass()))
			{
				const TAttribute<FText> Label = LOCTEXT("AnimSequence_NewAnimStreamable", "Create AnimStreamable");
				const TAttribute<FText> ToolTip = LOCTEXT("AnimSequence_NewAnimStreamableTooltip", "Creates an AnimStreamable using the selected anim sequence.");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimMontage");

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewAnimStreamable);
				Section.AddMenuEntry("AnimSequence_NewAnimStreamable", Label, ToolTip, Icon, UIAction);
			}
		}

		if (AssetTools.IsAssetClassSupported(UPoseAsset::StaticClass()))
		{
			const TAttribute<FText> Label = LOCTEXT("AnimSequence_NewPoseAsset", "Create PoseAsset");
			const TAttribute<FText> ToolTip = LOCTEXT("AnimSequence_NewPoseAssetTooltip", "Creates an PoseAsset using the selected anim sequence.");
			const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PoseAsset");

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewPoseAsset);
			Section.AddMenuEntry("AnimSequence_NewPoseAsset", Label, ToolTip, Icon, UIAction);
		}
	}

	void ExecuteReimportWithNewSource(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UObject*> ReimportAssets = CBContext->LoadSelectedObjects<UObject>();
	
		const bool bShowNotification = !FApp::IsUnattended();
		const bool bReimportWithNewFile = true;
		const int32 SourceFileIndex = INDEX_NONE;
		FReimportManager::Instance()->ValidateAllSourceFileAndReimport(ReimportAssets, bShowNotification, SourceFileIndex, bReimportWithNewFile);
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UAnimSequence::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				IAssetTools& AssetTools = IAssetTools::Get();

				if (AssetTools.IsAssetClassSupported(UAnimComposite::StaticClass()) ||
					AssetTools.IsAssetClassSupported(UAnimMontage::StaticClass()) ||
					AssetTools.IsAssetClassSupported(UAnimStreamable::StaticClass()) ||
					AssetTools.IsAssetClassSupported(UPoseAsset::StaticClass()))
				{
					InSection.AddSubMenu(
						"AnimSequence_CreateAnimSubmenu",
						LOCTEXT("CreateAnimSubmenu", "Create"),
						LOCTEXT("CreateAnimSubmenu_ToolTip", "Create assets from this anim sequence"),
						FNewToolMenuDelegate::CreateStatic(&FillCreateMenu),
						false,
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.CreateAnimAsset")
					);
				}
				{
					const TAttribute<FText> Label = LOCTEXT("AnimSequence_ReimportWithNewSource", "Reimport with New Source");
					const TAttribute<FText> ToolTip = LOCTEXT("AnimSequence_ReimportWithNewSourceTooltip", "Reimport the selected sequence(s) from a new source file.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.ReimportAnim");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteReimportWithNewSource);
					InSection.AddMenuEntry("AnimSequence_ReimportWithNewSource", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
