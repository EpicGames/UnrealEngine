// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimSequence.h"
#include "Animation/AssetDefinition_AnimationAsset.h"

#include "AssetDefinition_AnimSequence.generated.h"

class UAssetDefinition_AnimationAsset;

UCLASS()
class UAssetDefinition_AnimSequence : public UAssetDefinition_AnimationAsset
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimSequence", "Animation Sequence"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimSequence::StaticClass(); }
	virtual bool CanImport() const override { return true; }
	// UAssetDefinition End
};
