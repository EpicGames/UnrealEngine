// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimStreamable.h"
#include "Animation/AssetDefinition_AnimationAsset.h"

#include "AssetDefinition_AnimStreamable.generated.h"

class UAssetDefinition_AnimationAsset;

UCLASS()
class UAssetDefinition_AnimStreamable : public UAssetDefinition_AnimationAsset
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimStreamable", "Streamable Animation"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(181, 230, 29)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimStreamable::StaticClass(); }
	// UAssetDefinition End
};
