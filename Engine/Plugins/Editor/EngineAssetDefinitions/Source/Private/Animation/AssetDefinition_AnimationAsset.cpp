// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AnimationAsset.h"
#include "IAnimationEditor.h"
#include "Misc/MessageDialog.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "IAssetTools.h"
#include "ContentBrowserMenuContexts.h"
#include "SSkeletonWidget.h"
#include "IAnimationEditorModule.h"
#include "Preferences/PersonaOptions.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Framework/Application/SlateApplication.h"
#include "IPersonaToolkit.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_AnimationAsset)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

namespace UE::AnimationAsset
{
	bool ReplaceMissingSkeleton(const TArray<TWeakObjectPtr<UAnimationAsset>>& InAnimationAssets)
	{
		// record anim assets that need skeleton replaced
		const TArray<TWeakObjectPtr<UObject>> AnimsToFix = TArray<TWeakObjectPtr<UObject>>(InAnimationAssets);
		
		// get a skeleton from the user and replace it
		const TSharedPtr<SReplaceMissingSkeletonDialog> PickSkeletonWindow = SNew(SReplaceMissingSkeletonDialog).AnimAssets(AnimsToFix);
		const bool bWasSkeletonReplaced = PickSkeletonWindow.Get()->ShowModal();
		return bWasSkeletonReplaced;
	}
	
	void OpenAnimAssetEditor(const TArrayView<UAnimationAsset*>& InAnimAssets, bool bForceNewEditor, EToolkitMode::Type Mode, TSharedPtr<IToolkitHost> ToolkitHost)
	{
#if WITH_EDITOR
		// Force opening a new tab when the Animation Editor options window has the 'Always Open In New Tab' option enabled.
		const UPersonaOptions* PersonaOptions = GetDefault<UPersonaOptions>();
		check(PersonaOptions);
		bForceNewEditor |= PersonaOptions->bAlwaysOpenAnimationAssetsInNewTab;
#endif

		// Force new tab when shift is held down.
		bForceNewEditor |= FSlateApplication::Get().GetModifierKeys().IsShiftDown();

		// For each one..
		for (UAnimationAsset* AnimAsset : InAnimAssets)
		{
			USkeleton* AnimSkeleton = AnimAsset->GetSkeleton();
			if (!AnimSkeleton)
			{
				FText ShouldRetargetMessage = LOCTEXT("ShouldRetargetAnimAsset_Message", "Could not find the skeleton for Anim '{AnimName}' Would you like to choose a new one?");

				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("AnimName"), FText::FromString(AnimAsset->GetName()));

				if (FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(ShouldRetargetMessage, Arguments)) == EAppReturnType::Yes)
				{
					TArray<TWeakObjectPtr<UAnimationAsset>> AssetsToRetarget;
					AssetsToRetarget.Add(AnimAsset);
					const bool bSkeletonReplaced = ReplaceMissingSkeleton(AssetsToRetarget);
					if (!bSkeletonReplaced)
					{
						return; // Persona will crash if trying to load asset without a skeleton
					}
				}
				else
				{
					return;
				}
			}
		
			// First see if we already have it open
			const bool bBringToFrontIfOpen = true;
#if WITH_EDITOR
			if (IAssetEditorInstance* EditorInstance = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimAsset, bBringToFrontIfOpen))
			{
				EditorInstance->FocusWindow();
			}
			else
#endif
			{
				// See if we are trying to open a single asset. If we are, we re-use a compatible anim editor.
				bool bSingleAsset = InAnimAssets.Num() == 1;
				bool bFoundEditor = false;
				if (bSingleAsset && !bForceNewEditor)
				{
					// See if there is an animation asset with the same skeleton already being edited
					TArray<UObject*> AllEditedAssets;
#if WITH_EDITOR
					AllEditedAssets = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetAllEditedAssets();
#endif
					UAnimationAsset* CompatibleEditedAsset = nullptr;
					for (UObject* EditedAsset : AllEditedAssets)
					{
						UAnimationAsset* EditedAnimAsset = Cast<UAnimationAsset>(EditedAsset);
						if (EditedAnimAsset && EditedAnimAsset->GetSkeleton() == AnimAsset->GetSkeleton())
						{
							CompatibleEditedAsset = EditedAnimAsset;
							break;
						}
					}

					// If there is..
					if(CompatibleEditedAsset)
					{
						// Find the anim editors that are doing it
						TArray<IAssetEditorInstance*> AssetEditors;
#if WITH_EDITOR
						AssetEditors = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorsForAsset(CompatibleEditedAsset);
#endif
						for (IAssetEditorInstance* ExistingEditor : AssetEditors)
						{
							if (ExistingEditor->GetEditorName() == FName("AnimationEditor"))
							{
								// Change the current anim to this one
								IAnimationEditor* AnimEditor = static_cast<IAnimationEditor*>(ExistingEditor);
								if(AnimEditor->GetPersonaToolkit()->GetSkeleton() == AnimAsset->GetSkeleton())
								{
									AnimEditor->SetAnimationAsset(AnimAsset);
									AnimEditor->FocusWindow();
									bFoundEditor = true;
									break;
								}
							}
						}
					}
				}

				// We didn't find an editor, make a new one
				if (!bFoundEditor)
				{
					IAnimationEditorModule& AnimationEditorModule = FModuleManager::LoadModuleChecked<IAnimationEditorModule>("AnimationEditor");
					AnimationEditorModule.CreateAnimationEditor(Mode, ToolkitHost, AnimAsset);
				}
			}
		}
	}
}

EAssetCommandResult UAssetDefinition_AnimationAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	UE::AnimationAsset::OpenAnimAssetEditor(OpenArgs.LoadObjects<UAnimationAsset>(), false, OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost);

	return EAssetCommandResult::Handled;
}

UThumbnailInfo* UAssetDefinition_AnimationAsset::LoadThumbnailInfo(const FAssetData& InAssetData) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAssetData.GetAsset(), USceneThumbnailInfo::StaticClass());
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_AnimationAsset
{
	void ExecuteOpenInNewWindow(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UAnimationAsset*> AnimationAssets = Context->LoadSelectedObjects<UAnimationAsset>();
		
		UE::AnimationAsset::OpenAnimAssetEditor(AnimationAssets, true, EToolkitMode::Standalone, nullptr);
	}

	void ExecuteFindSkeleton(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		TArray<UObject*> ObjectsToSync;
		for (UAnimationAsset* AnimationAsset : Context->LoadSelectedObjects<UAnimationAsset>())
		{
			if (USkeleton* Skeleton = AnimationAsset->GetSkeleton())
			{
				ObjectsToSync.AddUnique(Skeleton);
			}
		}
		
		if ( ObjectsToSync.Num() > 0 )
		{
			IAssetTools::Get().SyncBrowserToAssets(ObjectsToSync);
		}
	}

	void ExecuteReplaceSkeleton(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UAnimationAsset*> AnimationAssets = Context->LoadSelectedObjects<UAnimationAsset>();
		
		UE::AnimationAsset::ReplaceMissingSkeleton(TArray<TWeakObjectPtr<UAnimationAsset>>(AnimationAssets));
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UAnimationAsset::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("AnimSequenceBase_OpenInNewWindow", "Open In New Window");
					const TAttribute<FText> ToolTip = LOCTEXT("AnimSequenceBase_OpenInNewWindowTooltip", "Will always open asset in a new window, and not re-use existing window. (Shift+Double-Click)");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.OpenInExternalEditor");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteOpenInNewWindow);
					InSection.AddMenuEntry("AnimSequenceBase_OpenInNewWindow", Label, ToolTip, Icon, UIAction);
				}
				{
					const TAttribute<FText> Label = LOCTEXT("AnimSequenceBase_FindSkeleton", "Find Skeleton");
					const TAttribute<FText> ToolTip = LOCTEXT("AnimSequenceBase_FindSkeletonTooltip", "Finds the skeleton for the selected assets in the content browser.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Find");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindSkeleton);
					InSection.AddMenuEntry("AnimSequenceBase_FindSkeleton", Label, ToolTip, Icon, UIAction);
				}
				{
					const TAttribute<FText> Label = LOCTEXT("AnimSequenceBase_ReplaceSkeleton", "Replace Skeleton");
					const TAttribute<FText> ToolTip = LOCTEXT("AnimSequenceBase_ReplaceSkeletonTooltip", "Associate a different skeleton with the selected animation assets in the content browser.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.RetargetManager");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteReplaceSkeleton);
					InSection.AddMenuEntry("AnimSequenceBase_ReplaceSkeleton", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
