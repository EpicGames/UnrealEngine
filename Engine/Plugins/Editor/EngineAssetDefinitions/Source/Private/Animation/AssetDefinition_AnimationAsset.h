// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimationAsset.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_AnimationAsset.generated.h"

UCLASS()
class UAssetDefinition_AnimationAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimationAsset", "AnimationAsset"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(80, 123, 72)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimationAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Animation };
		return Categories;
	}
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
