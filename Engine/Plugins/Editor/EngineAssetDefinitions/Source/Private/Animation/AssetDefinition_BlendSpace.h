// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AssetDefinition_AnimationAsset.h"
#include "Animation/BlendSpace.h"

#include "AssetDefinition_BlendSpace.generated.h"

class UAssetDefinition_AnimationAsset;

UCLASS()
class UAssetDefinition_BlendSpace : public UAssetDefinition_AnimationAsset
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_BlendSpace", "Blend Space"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 168, 111)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UBlendSpace::StaticClass(); }
	// UAssetDefinition End
};
