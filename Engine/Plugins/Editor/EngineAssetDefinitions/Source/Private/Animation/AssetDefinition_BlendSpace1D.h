// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AssetDefinition_AnimationAsset.h"
#include "Animation/BlendSpace1D.h"

#include "AssetDefinition_BlendSpace1D.generated.h"

class UAssetDefinition_AnimationAsset;

UCLASS()
class UAssetDefinition_BlendSpace1D : public UAssetDefinition_AnimationAsset
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_BlendSpace1D", "Blend Space 1D"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 180, 130)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UBlendSpace1D::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Animation / NSLOCTEXT("AssetTypeActions", "AnimLegacySubMenu", "Legacy") };
		return Categories;
	}
	// UAssetDefinition End
};
