// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PoseAsset.h"
#include "Animation/Skeleton.h"
#include "ToolMenuSection.h"
#include "Animation/AnimSequence.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ContentBrowserMenuContexts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PoseAsset)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_PoseAsset
{
	void ExecuteUpdateSource(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		FScopedTransaction Transaction(LOCTEXT("PoseUpdateSource", "Updating Source Animation for Pose"));
		for (UPoseAsset* PoseAsset : CBContext->LoadSelectedObjects<UPoseAsset>())
		{
			if (UAnimSequence* SourceAnimation = PoseAsset->SourceAnimation)
			{
				if (IAnimationDataModel* SourceAnimationDataModel = SourceAnimation->GetDataModel())
				{
					if (PoseAsset->SourceAnimationRawDataGUID.IsValid() && PoseAsset->SourceAnimationRawDataGUID != PoseAsset->GetSourceAnimationGuid())
					{
						if (PoseAsset->GetSkeleton()->IsCompatibleForEditor(SourceAnimation->GetSkeleton()))
						{
							PoseAsset->Modify();
							PoseAsset->UpdatePoseFromAnimation(SourceAnimation);
						}
					}
				}
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UPoseAsset::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection);

				{
					const TAttribute<FText> Label = LOCTEXT("PoseAsset_UpdateSource", "Update Source Animation");
					const TAttribute<FText> ToolTip = LOCTEXT("PoseAsset_UpdateSourceTooltip", "Updates the source animation for this pose");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimMontage");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteUpdateSource);
					InSection.AddMenuEntry("PoseAsset_UpdateSource", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
