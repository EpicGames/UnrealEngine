// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AssetDefinition_AnimationAsset.h"
#include "Animation/PoseAsset.h"

#include "AssetDefinition_PoseAsset.generated.h"

class UAssetDefinition_AnimationAsset;

UCLASS()
class UAssetDefinition_PoseAsset : public UAssetDefinition_AnimationAsset
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_PoseAsset", "Pose Asset"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(237, 28, 36)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPoseAsset::StaticClass(); }
	// UAssetDefinition End
};
