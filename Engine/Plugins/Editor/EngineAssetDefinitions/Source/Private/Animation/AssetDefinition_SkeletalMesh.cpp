// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AssetDefinition_SkeletalMesh.h"

#include "AnimationEditorUtils.h"
#include "AssetDefinitionAssetInfo.h"
#include "AssetNotifications.h"
#include "ContentBrowserMenuContexts.h"
#include "EditorReimportHandler.h"
#include "FbxMeshUtils.h"
#include "IAssetStatusInfoProvider.h"
#include "ISkeletalMeshEditorModule.h"
#include "Animation/SkeletalMeshUtilities.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Factories/SkeletonFactory.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Misc/MessageDialog.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/PhysicsAssetFactory.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "ToolMenuSection.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_SkeletalMesh)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UAssetDefinition_SkeletalMesh::UAssetDefinition_SkeletalMesh()
{
	//No need to remove since the asset registry module clear this multicast delegate when terminating
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRemoved().AddUObject(this, &UAssetDefinition_SkeletalMesh::OnAssetRemoved);
}

EAssetCommandResult UAssetDefinition_SkeletalMesh::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (USkeletalMesh* Mesh : OpenArgs.LoadObjects<USkeletalMesh>())
	{
		if (Mesh->GetSkeleton() == nullptr)
		{
			if ( FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("MissingSkeleton", "This mesh currently has no valid Skeleton. Would you like to create a new Skeleton?")) == EAppReturnType::Yes )
			{
				FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
				
				const FString DefaultSuffix = TEXT("_Skeleton");

				// Determine an appropriate name
				FString Name;
				FString PackageName;
				AssetToolsModule.Get().CreateUniqueAssetName(Mesh->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

				USkeletonFactory* Factory = NewObject<USkeletonFactory>();
				Factory->TargetSkeletalMesh = Mesh;
				
				AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), USkeleton::StaticClass(), Factory);
			}
			else
			{
				UE::SkeletalMeshUtilities::AssignSkeletonToMesh(Mesh);
			}

			if( Mesh->GetSkeleton() == nullptr )
			{
				// error message
				FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("CreateSkeletonOrAssign", "You need to create a Skeleton or assign one in order to open this in Persona."));
			}
		}

		if ( Mesh->GetSkeleton() != nullptr )
		{
			ISkeletalMeshEditorModule& SkeletalMeshEditorModule = FModuleManager::LoadModuleChecked<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
			SkeletalMeshEditorModule.CreateSkeletalMeshEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Mesh);
		}
	}
	
	return EAssetCommandResult::Handled;
}

void UAssetDefinition_SkeletalMesh::GetAssetStatusInfo(const TSharedPtr<IAssetStatusInfoProvider>& InAssetStatusInfoProvider, TArray<FAssetDisplayInfo>& OutStatusInfo) const
{
	Super::GetAssetStatusInfo(InAssetStatusInfoProvider, OutStatusInfo);

	const FAssetData AssetData = InAssetStatusInfoProvider->TryGetAssetData();
	if (AssetData.IsValid())
	{
		ThumbnailSkinningOverlayAssetNames.AddUnique(AssetData.GetFullName());
	}

	FAssetDisplayInfo SkinningStatus;
	SkinningStatus.StatusIcon = FAppStyle::GetBrush("ClassThumbnailOverlays.SkeletalMesh_NeedSkinning");
	SkinningStatus.Priority = FAssetStatusPriority(EStatusSeverity::Warning, 1);
	SkinningStatus.StatusDescription = LOCTEXT("AssetDefintion_SkeletalMesh_NeedSkinningStatus_ToolTip", "Asset geometry was imported, the skinning needs to be validated");
	SkinningStatus.IsVisible = MakeAttributeUObject(this, &UAssetDefinition_SkeletalMesh::GetThumbnailSkinningStatusVisibility, InAssetStatusInfoProvider);
	OutStatusInfo.Add(SkinningStatus);
}

UThumbnailInfo* UAssetDefinition_SkeletalMesh::LoadThumbnailInfo(const FAssetData& InAssetData) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAssetData.GetAsset(), USceneThumbnailInfo::StaticClass());
}

EVisibility UAssetDefinition_SkeletalMesh::GetThumbnailSkinningOverlayVisibility(const FAssetData AssetData) const
{
	//If the asset was delete it will be remove from the list, in that case do not use the
	//asset since it can be invalid if the GC has collect the object point by the AssetData.
	if (!ThumbnailSkinningOverlayAssetNames.Contains(AssetData.GetFullName()))
	{
		return EVisibility::Collapsed;
	}

	//Prevent loading assets when we display the thumbnail use the asset registry tags

	//Legacy fbx code
	FAssetDataTagMapSharedView::FFindTagResult Result = AssetData.TagsAndValues.FindTag(GET_MEMBER_NAME_CHECKED(UFbxSkeletalMeshImportData, LastImportContentType));
	if (Result.IsSet() && Result.GetValue() == TEXT("FBXICT_Geometry"))
	{
		//Show the icon
		return EVisibility::HitTestInvisible;
	}

#if WITH_EDITORONLY_DATA
	//Generic Interchange code
	Result = AssetData.TagsAndValues.FindTag(NSSkeletalMeshSourceFileLabels::GetSkeletalMeshLastImportContentTypeMetadataKey());
	if (Result.IsSet() && Result.GetValue().Equals(NSSkeletalMeshSourceFileLabels::GeometryMetaDataValue()))
	{
		//Show the icon
		return EVisibility::HitTestInvisible;
	}
#endif
	
	return EVisibility::Collapsed;
}

EVisibility UAssetDefinition_SkeletalMesh::GetThumbnailSkinningStatusVisibility(const TSharedPtr<IAssetStatusInfoProvider> InAssetStatusInfoProvider) const
{
	const FAssetData AssetData = InAssetStatusInfoProvider->TryGetAssetData();
	if (AssetData.IsValid())
	{
		return GetThumbnailSkinningOverlayVisibility(AssetData);
	}
	return EVisibility::Collapsed;
}

void UAssetDefinition_SkeletalMesh::OnAssetRemoved(const FAssetData& AssetData) const
{
	//Remove the object from the list before it get garbage collect
	if (ThumbnailSkinningOverlayAssetNames.Contains(AssetData.GetFullName()))
	{
		ThumbnailSkinningOverlayAssetNames.Remove(AssetData.GetFullName());
	}
}

TSharedPtr<SWidget> UAssetDefinition_SkeletalMesh::GetThumbnailOverlay(const FAssetData& AssetData) const
{
	const FSlateBrush* Icon = FAppStyle::GetBrush("ClassThumbnailOverlays.SkeletalMesh_NeedSkinning");

	ThumbnailSkinningOverlayAssetNames.AddUnique(AssetData.GetFullName());

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetNoBrush())
		.Visibility_UObject(this, &UAssetDefinition_SkeletalMesh::GetThumbnailSkinningOverlayVisibility, AssetData)
		.Padding(FMargin(0.0f, 0.0f, 3.0f, 3.0f))
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		[
			SNew(SImage)
			.ToolTipText(LOCTEXT("FAssetTypeActions_SkeletalMesh_NeedSkinning_ToolTip", "Asset geometry was imported, the skinning need to be validate"))
			.Image(Icon)
		];
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_SkeletalMesh
{
	void ExecuteNewPhysicsAsset(const FToolMenuContext& InContext, bool bSetAssetToMesh)
	{
		TArray<UObject*> CreatedObjects;
		
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		for (USkeletalMesh* SkeletalMesh : Context->LoadSelectedObjects<USkeletalMesh>())
		{
			if (SkeletalMesh->GetOutermost()->bIsCookedForEditor)
			{
				FAssetNotifications::CannotEditCookedAsset(SkeletalMesh);
				continue;
			}
			
			if (UObject* PhysicsAsset = UPhysicsAssetFactory::CreatePhysicsAssetFromMesh(NAME_None, nullptr, SkeletalMesh, bSetAssetToMesh))
			{
				CreatedObjects.Add(PhysicsAsset);
			}
		}
    
		if(CreatedObjects.Num() > 0)
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToAssets(CreatedObjects);
#if WITH_EDITOR
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(CreatedObjects);
#endif
		}
	}

	void GetPhysicsAssetMenu(UToolMenu* Menu)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*Menu);

    	FToolMenuSection& Section = Menu->FindOrAddSection("PhysicsActions");
		
		{
			const TAttribute<FText> Label = LOCTEXT("PhysAsset_Create", "Create");
			const TAttribute<FText> ToolTip = LOCTEXT("PhysAsset_Create_ToolTip", "Create new physics assets without assigning it to the selected meshes");
			const FSlateIcon Icon = FSlateIcon();

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewPhysicsAsset, false);
			Section.AddMenuEntry("PhysAsset_Create", Label, ToolTip, Icon, UIAction);
		}
		
		{
			const TAttribute<FText> Label = LOCTEXT("PhysAsset_CreateAssign", "Create and Assign");
			const TAttribute<FText> ToolTip = LOCTEXT("PhysAsset_CreateAssign_ToolTip", "Create new physics assets and assign it to each of the selected meshes");
			const FSlateIcon Icon = FSlateIcon();

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewPhysicsAsset, true);
			Section.AddMenuEntry("PhysAsset_CreateAssign", Label, ToolTip, Icon, UIAction);
		}
	}

	bool OnAssetCreated(TArray<UObject*> NewAssets)
	{
		if (NewAssets.Num() > 1)
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
			AssetToolsModule.Get().SyncBrowserToAssets(NewAssets);
		}
		return true;
	}
	
    void FillCreateMenu(UToolMenu* Menu)
    {
    	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
    	if (AssetTools.IsAssetClassSupported(UPhysicsAsset::StaticClass()))
    	{
    		FToolMenuSection& Section = Menu->AddSection("CreatePhysicsAsset", LOCTEXT("CreatePhysicsAssetMenuHeading", "Physics Asset"));
    		Section.AddSubMenu(
    			"NewPhysicsAssetMenu",
    			LOCTEXT("SkeletalMesh_NewPhysicsAssetMenu", "Physics Asset"),
    			LOCTEXT("SkeletalMesh_NewPhysicsAssetMenu_ToolTip", "Options for creating new physics assets from the selected meshes."),
    			FNewToolMenuDelegate::CreateStatic(&GetPhysicsAssetMenu));
    	}

    	Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegateLegacy::CreateLambda([](FMenuBuilder& MenuBuilder, UToolMenu* Menu)
    	{
    		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*Menu);
			const TArray<TSoftObjectPtr<UObject>> SkeletalMeshes = Context->GetSelectedAssetSoftObjects<UObject>();
    		
    		AnimationEditorUtils::FillCreateAssetMenu(MenuBuilder, SkeletalMeshes, FAnimAssetCreated::CreateStatic(&OnAssetCreated));
    	}));
    }
    
    void GetResolvedSourceFilePaths(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFilePaths)
    {
    	for (auto& Asset : TypeAssets)
    	{
    		const auto SkeletalMesh = CastChecked<USkeletalMesh>(Asset);
    		SkeletalMesh->GetAssetImportData()->ExtractFilenames(OutSourceFilePaths);
    	}
    }
    
    void GetSourceFileLabels(const TArray<UObject*>& TypeAssets, TArray<FString>& OutSourceFileLabels)
    {
    	for (auto& Asset : TypeAssets)
    	{
    		const auto SkeletalMesh = CastChecked<USkeletalMesh>(Asset);
    		TArray<FString> SourceFilePaths;
    		SkeletalMesh->GetAssetImportData()->ExtractFilenames(SourceFilePaths);
    		for (int32 SourceIndex = 0; SourceIndex < SourceFilePaths.Num(); ++SourceIndex)
    		{
    			FText SourceIndexLabel = USkeletalMesh::GetSourceFileLabelFromIndex(SourceIndex);
    			OutSourceFileLabels.Add(SourceIndexLabel.ToString());
    		}
    	}
    }

	void ExecuteImportMeshLOD(const FToolMenuContext&, TSoftObjectPtr<USkeletalMesh> SkeletalMeshPtr, int32 LOD)
	{
		if (USkeletalMesh* SkeletalMesh = SkeletalMeshPtr.LoadSynchronous())
		{
			if (SkeletalMesh->GetOutermost()->bIsCookedForEditor)
			{
				FAssetNotifications::CannotEditCookedAsset(SkeletalMesh);
				return;
			}
    
			if (LOD == 0)
			{
				//re-import of the asset
				TArray<UObject*> AssetArray;
				AssetArray.Add(SkeletalMesh);
				FReimportManager::Instance()->ValidateAllSourceFileAndReimport(AssetArray);
			}
			else
			{
				FbxMeshUtils::ImportMeshLODDialog(SkeletalMesh, LOD);
			}
		}
	}

	const static FName NAME_LODs("LODs");
	static int32 GetNumberOfLODs(const FAssetData& SkeletalMeshAsset)
	{
		int32 NumberOfLODs = 0;
		return SkeletalMeshAsset.GetTagValue<int32>(NAME_LODs, NumberOfLODs) ? NumberOfLODs : 0;
	}

	// Nanite Section
	//=================================================================
	
	const static FName NAME_NaniteEnabled("NaniteEnabled");
	static bool IsNaniteEnabled(const FAssetData& SkeletalMeshAsset)
	{
		bool bNaniteEnabled = false;
		return SkeletalMeshAsset.GetTagValue<bool>(NAME_NaniteEnabled, bNaniteEnabled) && bNaniteEnabled;
	}
	
	void ModifyNaniteEnable(const TArray<USkeletalMesh*>& Objects, bool bNaniteEnable)
	{
		TArray<USkeletalMesh*> Meshes;
		Meshes.Reserve(Objects.Num());

		for (USkeletalMesh* Mesh: Objects)
		{
			if (Mesh && Mesh->NaniteSettings.bEnabled != bNaniteEnable)
			{
				Meshes.Add(Mesh);
			}
		}

		for (USkeletalMesh* Mesh : Meshes)
		{
			Mesh->NaniteSettings.bEnabled = bNaniteEnable;
			Mesh->Build();
			Mesh->MarkPackageDirty();
		}
	}

	void ExecuteNaniteEnable(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<USkeletalMesh*> RegularMeshes = Context->LoadSelectedObjectsIf<USkeletalMesh>([](const FAssetData& AssetData)
		{
			return !IsNaniteEnabled(AssetData);
		});
		ModifyNaniteEnable(RegularMeshes, true);
	}

	void ExecuteNaniteDisable(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<USkeletalMesh*> NaniteMeshes = Context->LoadSelectedObjectsIf<USkeletalMesh>([](const FAssetData& AssetData)
		{
			return IsNaniteEnabled(AssetData);
		});
		ModifyNaniteEnable(NaniteMeshes, false);
	}

	static void GetNaniteMenu(UToolMenu* Menu)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*Menu);
	
		FToolMenuSection& Section = Menu->FindOrAddSection("NaniteActions");

		{
			const TAttribute<FText> Label = LOCTEXT("SkeletalMesh_NaniteToggle", "Nanite");
			const TAttribute<FText> ToolTip = LOCTEXT("SkeletalMesh_NaniteToggleTooltip", "Toggle Nanite support on the selected meshes.");
			const FSlateIcon Icon = FSlateIcon();

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
				{
					const bool bContainsTrue = Context->SelectedAssets.ContainsByPredicate([](const FAssetData& InAsset) { return IsNaniteEnabled(InAsset); });
					const bool bContainsFalse = Context->SelectedAssets.ContainsByPredicate([](const FAssetData& InAsset) { return !IsNaniteEnabled(InAsset); });

					if ((bContainsTrue && bContainsFalse) || bContainsTrue)
					{
						ExecuteNaniteDisable(InContext);
					}
					else
					{
						ExecuteNaniteEnable(InContext);
					}
				}
			});
			UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([](const FToolMenuContext& InContext)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
				{
					const bool bContainsTrue = Context->SelectedAssets.ContainsByPredicate([](const FAssetData& InAsset) { return IsNaniteEnabled(InAsset); });
					const bool bContainsFalse = Context->SelectedAssets.ContainsByPredicate([](const FAssetData& InAsset) { return !IsNaniteEnabled(InAsset); });
					return bContainsTrue && bContainsFalse ? ECheckBoxState::Undetermined : (bContainsTrue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
				}

				return ECheckBoxState::Undetermined;
			});
			
			Section.AddMenuEntry("SkeletalMesh_NaniteToggle", Label, ToolTip, Icon, UIAction, EUserInterfaceActionType::ToggleButton);
		}

		Section.AddSeparator("Nanite_EnableDisableOptions");

		const int32 NaniteMeshes = Algo::Accumulate(Context->SelectedAssets, 0, [](int32 Value, const FAssetData& SkeletalMeshAsset)
		{
			bool bNaniteEnabled = false;
			return (SkeletalMeshAsset.GetTagValue<bool>(NAME_NaniteEnabled, bNaniteEnabled) && bNaniteEnabled) ? Value + 1 : Value;
		});

		{
			const int32 RegularMeshes = Context->SelectedAssets.Num() - NaniteMeshes;
			
			const TAttribute<FText> Label = FText::Format(LOCTEXT("SkeletalMesh_NaniteEnableAll", "Enable Nanite ({0} Meshes)"), RegularMeshes);
			const TAttribute<FText> ToolTip = LOCTEXT("SkeletalMesh_NaniteEnableAllTooltip", "Enables support for Nanite on the selected meshes.");
			const FSlateIcon Icon = FSlateIcon();
			
			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNaniteEnable);
			UIAction.CanExecuteAction.BindLambda([RegularMeshes](const FToolMenuContext&) { return RegularMeshes > 0; });
			Section.AddMenuEntry("SkeletalMesh_EnableNanite", Label, ToolTip, Icon, UIAction);
		}
		
		{
			const TAttribute<FText> Label = FText::Format(LOCTEXT("SkeletalMesh_NaniteDisableAll", "Disable Nanite ({0} Meshes)"), NaniteMeshes);
			const TAttribute<FText> ToolTip = LOCTEXT("SkeletalMesh_NaniteDisableAllTooltip", "Disables support for Nanite on the selected meshes.");
			const FSlateIcon Icon = FSlateIcon();
			
			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNaniteDisable);
			UIAction.CanExecuteAction.BindLambda([NaniteMeshes](const FToolMenuContext&) { return NaniteMeshes > 0; });
			Section.AddMenuEntry("SkeletalMesh_DisableNanite", Label, ToolTip, Icon, UIAction);
		}
	}

    void GetLODMenu(UToolMenu* Menu)
    {
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*Menu);
		const FAssetData SkeletalMeshAsset = Context->SelectedAssets[0];
		TSoftObjectPtr<USkeletalMesh> SkeletalMeshPtr = TSoftObjectPtr<USkeletalMesh>(SkeletalMeshAsset.ToSoftObjectPath());

		FToolMenuSection& Section = Menu->FindOrAddSection("LODImport");
		
    	const int32 LODMax = GetNumberOfLODs(SkeletalMeshAsset);
    	for(int32 LOD = 1; LOD <= LODMax; ++LOD)
    	{
    		const TAttribute<FText> Label = (LOD == LODMax) ? FText::Format(LOCTEXT("AddLODLevel", "Add LOD {0}"), FText::AsNumber(LOD)) : FText::Format( LOCTEXT("LODLevel", "Reimport LOD {0}"), FText::AsNumber( LOD ) );
    		const TAttribute<FText> ToolTip = ( LOD == LODMax ) ? LOCTEXT("NewImportTip", "Import new LOD") : LOCTEXT("ReimportTip", "Reimport over existing LOD");
    		const FSlateIcon Icon = FSlateIcon();

    		FToolUIAction UIAction;
    		UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteImportMeshLOD, SkeletalMeshPtr, LOD);
    		Section.AddMenuEntry(*FString::Printf(TEXT("LOD_%d"), LOD), Label, ToolTip, Icon, UIAction);
    	}
    }
    
    void ExecuteNewSkeleton(const FToolMenuContext& InContext)
    {
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		IAssetTools::Get().CreateAssetsFrom<USkeletalMesh>(
			CBContext->LoadSelectedObjects<USkeletalMesh>(), USkeleton::StaticClass(), TEXT("_Skeleton"), [](USkeletalMesh* SourceObject)
			{
				if (SourceObject->GetOutermost()->bIsCookedForEditor)
				{
					FAssetNotifications::CannotEditCookedAsset(SourceObject);
					return static_cast<USkeletonFactory*>(nullptr);
				}
				
				USkeletonFactory* Factory = NewObject<USkeletonFactory>();
				Factory->TargetSkeletalMesh = SourceObject;
				
				return Factory;
			}
		);
    }
    
    void ExecuteAssignSkeleton(const FToolMenuContext& InContext)
    {
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		
		for (USkeletalMesh* SkeletalMesh : CBContext->LoadSelectedObjects<USkeletalMesh>())
		{
			UE::SkeletalMeshUtilities::AssignSkeletonToMesh(SkeletalMesh);
    	}
    }
    
    void ExecuteFindSkeleton(const FToolMenuContext& InContext)
    {
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

    	TArray<UObject*> ObjectsToSync;
    	for (USkeletalMesh* SkeletalMesh : Context->LoadSelectedObjects<USkeletalMesh>())
    	{
	        if (USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
    		{
    			ObjectsToSync.AddUnique(Skeleton);
    		}
    	}
    
    	if ( ObjectsToSync.Num() > 0 )
    	{
    		IAssetTools::Get().SyncBrowserToAssets(ObjectsToSync);
    	}
    }
    
    void FillSkeletonMenu(UToolMenu* Menu)
    {
		FToolMenuSection& Section = Menu->FindOrAddSection("SkeletonMenu");
		Section.Label = LOCTEXT("SkeletonMenuHeading", "Skeleton");
		
		{
			const TAttribute<FText> Label = LOCTEXT("SkeletalMesh_NewSkeleton", "Create Skeleton");
			const TAttribute<FText> ToolTip = LOCTEXT("SkeletalMesh_NewSkeletonTooltip", "Creates a new skeleton for each of the selected meshes.");
			const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetIcons.Skeleton");

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewSkeleton);
			Section.AddMenuEntry("SkeletalMesh_NewSkeleton", Label, ToolTip, Icon, UIAction);
		}
		
		{
			const TAttribute<FText> Label = LOCTEXT("SkeletalMesh_AssignSkeleton", "Assign Skeleton");
			const TAttribute<FText> ToolTip = LOCTEXT("SkeletalMesh_AssignSkeletonTooltip", "Assigns a skeleton to the selected meshes.");
			const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.AssignSkeleton");

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteAssignSkeleton);
			Section.AddMenuEntry("SkeletalMesh_AssignSkeleton", Label, ToolTip, Icon, UIAction);
		}
    
		{
			const TAttribute<FText> Label = LOCTEXT("SkeletalMesh_FindSkeleton", "Find Skeleton");
			const TAttribute<FText> ToolTip = LOCTEXT("SkeletalMesh_FindSkeletonTooltip", "Finds the skeleton used by the selected meshes in the content browser.");
			const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.FindSkeleton");

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindSkeleton);
			Section.AddMenuEntry("SkeletalMesh_FindSkeleton", Label, ToolTip, Icon, UIAction);
		}
    }
	
    
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
    	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
    	{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USkeletalMesh::StaticClass());
    			
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection);
				if (Context && Context->bCanBeModified)
				{
					{
						InSection.AddSubMenu(
							"CreateSkeletalMeshSubmenu",
							LOCTEXT("CreateSkeletalMeshSubmenu", "Create"),
							LOCTEXT("CreateSkeletalMeshSubmenu_ToolTip", "Create related assets"),
							FNewToolMenuDelegate::CreateStatic(&FillCreateMenu),
							false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.CreateAnimAsset")
						);
					}

					static const auto AllowSkinnedMeshes = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AllowSkinnedMeshes"));
					static const bool bAllowSkinnedMeshes = (AllowSkinnedMeshes && AllowSkinnedMeshes->GetValueOnAnyThread() != 0);
					if (bAllowSkinnedMeshes)
					{
						InSection.AddSubMenu(
							"SkeletalMesh_NaniteSubmenu",
							LOCTEXT("SkeletalMesh_NaniteMenu", "Nanite"),
							LOCTEXT("SkeletalMesh_NaniteTooltip", "Nanite Options and Tools"),
							FNewToolMenuDelegate::CreateStatic(&GetNaniteMenu),
							false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Adjust")
						);
					}

					if (UContentBrowserAssetContextMenuContext::GetNumAssetsSelected(InSection))
					{
						InSection.AddSubMenu(
							"SkeletalMesh_LODImport",	
							LOCTEXT("SkeletalMesh_LODImport", "Import LOD"),
							LOCTEXT("SkeletalMesh_LODImportTooltip", "Select which LODs to import."),
							FNewToolMenuDelegate::CreateStatic(&GetLODMenu),
							false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.LOD")
						);
					}

					{
						InSection.AddSubMenu(
							"SkeletonSubmenu",
							LOCTEXT("SkeletonSubmenu", "Skeleton"),
							LOCTEXT("SkeletonSubmenu_ToolTip", "Skeleton related actions"),
							FNewToolMenuDelegate::CreateStatic(&FillSkeletonMenu),
							false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.SkeletalMesh")
						);
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
