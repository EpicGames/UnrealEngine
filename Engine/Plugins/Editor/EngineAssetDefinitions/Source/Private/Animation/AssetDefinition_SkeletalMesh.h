// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/SkeletalMesh.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_SkeletalMesh.generated.h"

struct EVisibility;

UCLASS()
class UAssetDefinition_SkeletalMesh : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	UAssetDefinition_SkeletalMesh();
	
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SkeletalMesh", "Skeletal Mesh"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(241,163,241)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USkeletalMesh::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Basic, EAssetCategoryPaths::Animation };
		return Categories;
	}
	virtual void GetAssetStatusInfo(const TSharedPtr<IAssetStatusInfoProvider>& InAssetStatusInfoProvider, TArray<FAssetDisplayInfo>& OutStatusInfo) const override;
	virtual bool CanImport() const override { return true; }
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;
	virtual TSharedPtr<class SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End

private:
	void OnAssetRemoved(const FAssetData& AssetData) const;
	EVisibility GetThumbnailSkinningOverlayVisibility(const FAssetData AssetData) const;
	EVisibility GetThumbnailSkinningStatusVisibility(const TSharedPtr<IAssetStatusInfoProvider> InAssetStatusInfoProvider) const;

	mutable TArray<FString> ThumbnailSkinningOverlayAssetNames;
};
