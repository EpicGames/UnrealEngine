// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AssetDefinition_Skeleton.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "AnimationEditorUtils.h"
#include "ISkeletonEditorModule.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Skeleton)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_Skeleton::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	
	for (USkeleton* Skeleton : OpenArgs.LoadObjects<USkeleton>())
	{
		SkeletonEditorModule.CreateSkeletonEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Skeleton);
	}

	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_Skeleton
{
	static void GetCreateAssetsForSkeletonMenu(UToolMenu* Menu)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*Menu);
		TArray<TSoftObjectPtr<UObject>> Skeletons = Context->GetSelectedAssetSoftObjects<UObject>();

		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegateLegacy::CreateLambda([Skeletons](FMenuBuilder& MenuBuilder, UToolMenu* Menu)
		{
			AnimationEditorUtils::FillCreateAssetMenu(MenuBuilder, Skeletons, FAnimAssetCreated::CreateLambda([](TArray<UObject*> NewAssets)
				{
					if(NewAssets.Num() > 1)
					{
						FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get().SyncBrowserToAssets(NewAssets);
					}
					return true;
				})
			);
		}));
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USkeleton::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				InSection.AddSubMenu(
					"CreateSkeletonSubmenu",
					LOCTEXT("CreateSkeletonSubmenu", "Create"),
					LOCTEXT("CreateSkeletonSubmenu_ToolTip", "Create assets for this skeleton"),
					FNewToolMenuDelegate::CreateStatic(&GetCreateAssetsForSkeletonMenu),
					false, 
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.CreateAnimAsset")
				);
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
