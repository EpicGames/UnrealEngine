// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/Skeleton.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_Skeleton.generated.h"

UCLASS()
class UAssetDefinition_Skeleton : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Skeleton", "Skeleton"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(105,181,205)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USkeleton::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Animation };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
