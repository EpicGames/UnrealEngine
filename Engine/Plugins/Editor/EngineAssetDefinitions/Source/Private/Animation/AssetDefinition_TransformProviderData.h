// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/TransformProviderData.h"
#include "AssetDefinitionDefault.h"
#include "AssetDefinition_TransformProviderData.generated.h"

UCLASS()
class UAssetDefinition_TransformProviderData : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TransformProviderData", "Transform Provider Data"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(78, 40, 165)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTransformProviderData::StaticClass(); }
};
