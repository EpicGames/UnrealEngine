// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/SkeletalMeshUtilities.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/MessageDialog.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "SSkeletonWidget.h"
#include "AssetNotifications.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

namespace UE::SkeletalMeshUtilities
{
	/**
	* FBoneCheckboxInfo
	* 
	* Context data for the SDlgMergeSkeleton panel check boxes
	*/
	struct FBoneCheckboxInfo
	{
		FName BoneName;
		int32 BoneID;
		bool  bUsed;
	};

	/** 
	* FDlgMergeSkeleton
	* 
	* Wrapper class for SDlgMergeSkeleton. This class creates and launches a dialog then awaits the
	* result to return to the user. 
	*/
	class FDlgMergeSkeleton
	{
	public:
		enum EResult
		{
			Cancel = 0,			// No/Cancel, normal usage would stop the current action
			Confirm = 1,		// Yes/Ok/Etc, normal usage would continue with action
		};

		FDlgMergeSkeleton( USkeletalMesh* InMesh, USkeleton* InSkeleton );

		/**  Shows the dialog box and waits for the user to respond. */
		EResult ShowModal();

		// List of required bones for skeleton
		TArray<int32> RequiredBones;
	private:

		/** Cached pointer to the modal window */
		TSharedPtr<SWindow> DialogWindow;

		/** Cached pointer to the merge skeleton widget */
		TSharedPtr<class SDlgMergeSkeleton> DialogWidget;

		/** The SkeletalMesh to merge bones from*/
		USkeletalMesh* Mesh;
		/** The Skeleton to merge bones to */
		USkeleton* Skeleton;
	};

	/** 
	 * Slate panel for choosing which bones to merge into the skeleton
	 */
	class SDlgMergeSkeleton : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SDlgMergeSkeleton)
			{}
			/** Window in which this widget resides */
			SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		SLATE_END_ARGS()	

		/**
		 * Constructs this widget
		 *
		 * @param	InArgs	The declaration data for this widget
		 */
		BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
		void Construct( const FArguments& InArgs )
		{
			UserResponse = FDlgMergeSkeleton::Cancel;
			ParentWindow = InArgs._ParentWindow.Get();

			this->ChildSlot[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew( STextBlock )
					.Text( LOCTEXT("MergeSkeletonDlgDescription", "Would you like to add following bones to the skeleton?"))
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SSeparator)
				]
				+SVerticalBox::Slot()
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SBorder)
					[
						SNew(SScrollBox)
						+SScrollBox::Slot()
						[
							//Save this widget so we can populate it later with check boxes
							SAssignNew(CheckBoxContainer, SVerticalBox)
						]
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+SUniformGridPanel::Slot(0,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &SDlgMergeSkeleton::ChangeAllOptions, true)
						.Text(LOCTEXT("SkeletonMergeSelectAll", "Select All"))
					]
					+SUniformGridPanel::Slot(1,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &SDlgMergeSkeleton::ChangeAllOptions, false)
						.Text(LOCTEXT("SkeletonMergeDeselectAll", "Deselect All"))
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SSeparator)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+SUniformGridPanel::Slot(0,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked( this, &SDlgMergeSkeleton::OnButtonClick, FDlgMergeSkeleton::Confirm )
						.Text(LOCTEXT("SkeletonMergeOk", "OK"))
					]
					+SUniformGridPanel::Slot(1,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked( this, &SDlgMergeSkeleton::OnButtonClick, FDlgMergeSkeleton::Cancel )
						.Text(LOCTEXT("SkeletonMergeCancel", "Cancel"))
					]
				]
			];
		}
		END_SLATE_FUNCTION_BUILD_OPTIMIZATION

		/**
		 * Creates a Slate check box 
		 *
		 * @param	Label		Text label for the check box
		 * @param	ButtonId	The ID for the check box
		 * @return				The created check box widget
		 */
		TSharedRef<SWidget> CreateCheckBox( const FString& Label, int32 ButtonId )
		{
			return
				SNew(SCheckBox)
				.IsChecked( this, &SDlgMergeSkeleton::IsCheckboxChecked, ButtonId )
				.OnCheckStateChanged( this, &SDlgMergeSkeleton::OnCheckboxChanged, ButtonId )
				[
					SNew(STextBlock).Text(FText::FromString(Label))
				];
		}

		/**
		 * Returns the state of the check box
		 *
		 * @param	ButtonId	The ID for the check box
		 * @return				The status of the check box
		 */
		ECheckBoxState IsCheckboxChecked( int32 ButtonId ) const
		{
			return CheckBoxInfoMap.FindChecked(ButtonId).bUsed ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		/**
		 * Handler for all check box clicks
		 *
		 * @param	NewCheckboxState	The new state of the check box
		 * @param	CheckboxThatChanged	The ID of the radio button that has changed. 
		 */
		void OnCheckboxChanged( ECheckBoxState NewCheckboxState, int32 CheckboxThatChanged )
		{
			FBoneCheckboxInfo& Info = CheckBoxInfoMap.FindChecked(CheckboxThatChanged);
			Info.bUsed = !Info.bUsed;
		}

		/**
		 * Handler for the Select All and Deselect All buttons
		 *
		 * @param	bNewCheckedState	The new state of the check boxes
		 */
		FReply ChangeAllOptions(bool bNewCheckedState)
		{
			for(auto Iter = CheckBoxInfoMap.CreateIterator(); Iter; ++Iter)
			{
				FBoneCheckboxInfo& Info = Iter.Value();
				Info.bUsed = bNewCheckedState;
			}
			return FReply::Handled();
		}

		/**
		 * Populated the dialog with multiple check boxes, each corresponding to a bone
		 *
		 * @param	BoneInfos	The list of Bones to populate the dialog with
		 */
		void PopulateOptions(TArray<FBoneCheckboxInfo>& BoneInfos)
		{
			for(auto Iter = BoneInfos.CreateIterator(); Iter; ++Iter)
			{
				FBoneCheckboxInfo& Info = (*Iter);
				Info.bUsed = true;

				CheckBoxInfoMap.Add(Info.BoneID, Info);

				CheckBoxContainer->AddSlot()
				.AutoHeight()
				[
					CreateCheckBox(Info.BoneName.GetPlainNameString(), Info.BoneID)
				];
			}
		}

		/** 
		 * Returns the EResult of the button which the user pressed. Closing of the dialog
		 * in any other way than clicking "Ok" results in this returning a "Cancel" value
		 */
		FDlgMergeSkeleton::EResult GetUserResponse() const 
		{
			return UserResponse; 
		}

		/** 
		 * Returns whether the user selected that bone to be used (checked its respective check box)
		 */
		bool IsBoneIncluded(int32 BoneID)
		{
			auto* Item = CheckBoxInfoMap.Find(BoneID);
			return Item ? Item->bUsed : false;
		}

	private:
		
		/** 
		 * Handles when a button is pressed, should be bound with appropriate EResult Key
		 * 
		 * @param ButtonID - The return type of the button which has been pressed.
		 */
		FReply OnButtonClick(FDlgMergeSkeleton::EResult ButtonID)
		{
			ParentWindow->RequestDestroyWindow();
			UserResponse = ButtonID;

			return FReply::Handled();
		}

		/** Stores the users response to this dialog */
		FDlgMergeSkeleton::EResult	 UserResponse;

		/** The slate container that the bone check boxes get added to */
		TSharedPtr<SVerticalBox>	 CheckBoxContainer;
		/** Store the check box state for each bone */
		TMap<int32,FBoneCheckboxInfo> CheckBoxInfoMap;

		/** Pointer to the window which holds this Widget, required for modal control */
		TSharedPtr<SWindow>			 ParentWindow;
	};

	FDlgMergeSkeleton::FDlgMergeSkeleton( USkeletalMesh* InMesh, USkeleton* InSkeleton )
	{
		Mesh = InMesh;
		Skeleton = InSkeleton;

		if (FSlateApplication::IsInitialized())
		{
			DialogWindow = SNew(SWindow)
				.Title( LOCTEXT("MergeSkeletonDlgTitle", "Merge Bones") )
				.SupportsMinimize(false) .SupportsMaximize(false)
				.ClientSize(FVector2D(350, 500));

			TSharedPtr<SBorder> DialogWrapper = 
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(4.0f)
				[
					SAssignNew(DialogWidget, SDlgMergeSkeleton)
					.ParentWindow(DialogWindow)
				];

			DialogWindow->SetContent(DialogWrapper.ToSharedRef());
		}
	}

	FDlgMergeSkeleton::EResult FDlgMergeSkeleton::ShowModal()
	{
		RequiredBones.Empty();

		TMap<FName, int32> BoneIndicesMap;
		TArray<FBoneCheckboxInfo> BoneInfos;

		// Make a list of all skeleton bone list
		const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
		for ( int32 BoneTreeId=0; BoneTreeId<RefSkeleton.GetRawBoneNum(); ++BoneTreeId )
		{
			const FName& BoneName = RefSkeleton.GetBoneName(BoneTreeId);
			BoneIndicesMap.Add(BoneName, BoneTreeId);
		}

		for ( int32 RefBoneId=0 ; RefBoneId< Mesh->GetRefSkeleton().GetRawBoneNum() ; ++RefBoneId )
		{
			const FName& BoneName = Mesh->GetRefSkeleton().GetBoneName(RefBoneId);
			// if I can't find this from Skeleton
			if (BoneIndicesMap.Find(BoneName)==NULL)
			{
				FBoneCheckboxInfo Info;
				Info.BoneID = RefBoneId;
				Info.BoneName = BoneName;
				BoneInfos.Add(Info);
			}
		}

		if (BoneInfos.Num() == 0)
		{
			// it's all identical, but still need to return RequiredBones
			// for the case, where they'd like to replace the one exactly same hierarchy but different skeleton 
			for ( int32 RefBoneId= 0 ; RefBoneId< Mesh->GetRefSkeleton().GetRawBoneNum() ; ++RefBoneId )
			{
				const FName& BoneName = Mesh->GetRefSkeleton().GetBoneName(RefBoneId);
				RequiredBones.Add(RefBoneId);
			}

			return EResult::Confirm;
		}

		DialogWidget->PopulateOptions(BoneInfos);

		//Show Dialog
		GEditor->EditorAddModalWindow(DialogWindow.ToSharedRef());
		EResult UserResponse = (EResult)DialogWidget->GetUserResponse();

		if(UserResponse == EResult::Confirm)
		{
			for ( int32 RefBoneId= 0 ; RefBoneId< Mesh->GetRefSkeleton().GetRawBoneNum() ; ++RefBoneId )
			{
				if ( DialogWidget->IsBoneIncluded(RefBoneId) )
				{
					TArray<int32> ParentList;
					
					// I need to make sure parent exists first
					int32 ParentIndex = Mesh->GetRefSkeleton().GetParentIndex(RefBoneId);
					
					// make sure RequiredBones already have ParentIndex
					while (ParentIndex >= 0 )
					{
						// if I don't have it yet
						if ( RequiredBones.Contains(ParentIndex)==false )
						{
							ParentList.Add(ParentIndex);
						}

						ParentIndex = Mesh->GetRefSkeleton().GetParentIndex(ParentIndex);
					}

					if ( ParentList.Num() > 0 )
					{
						// if we need to add parent list
						// add from back to front (since it's added from child to up
						for (int32 I=ParentList.Num()-1; I>=0; --I)
						{
							RequiredBones.Add(ParentList[I]);
						}
					}

					RequiredBones.Add(RefBoneId);
				}
			}
		}
		return UserResponse;
	}

	void AssignSkeletonToMesh(USkeletalMesh* SkelMesh)
	{
		if (!SkelMesh || SkelMesh->GetOutermost()->bIsCookedForEditor)
		{
			FAssetNotifications::CannotEditCookedAsset(SkelMesh);
			return;
		}

		// Create a skeleton asset from the selected skeletal mesh. Defaults to being in the same package/group as the skeletal mesh.
		TSharedRef<SWindow> WidgetWindow = SNew(SWindow)
			.Title(LOCTEXT("ChooseSkeletonWindowTitle", "Choose Skeleton"))
			.ClientSize(FVector2D(400,600));
		TSharedPtr<SSkeletonSelectorWindow> SkeletonSelectorWindow;
		WidgetWindow->SetContent
			(
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SAssignNew(SkeletonSelectorWindow, SSkeletonSelectorWindow)
						.Object(SkelMesh)
						.WidgetWindow(WidgetWindow)
				]
			);

		GEditor->EditorAddModalWindow(WidgetWindow);
		USkeleton* SelectedSkeleton = SkeletonSelectorWindow->GetSelectedSkeleton();

		// only do this if not same
		if ( SelectedSkeleton )
		{
			FDlgMergeSkeleton AssetDlg( SkelMesh, SelectedSkeleton );
			if(AssetDlg.ShowModal() == FDlgMergeSkeleton::Confirm)
			{			
				TArray<int32> RequiredBones;
				RequiredBones.Append(AssetDlg.RequiredBones);

				if ( RequiredBones.Num() > 0 )
				{
					// Do automatic asset generation.
					bool bSuccess = SelectedSkeleton->MergeBonesToBoneTree( SkelMesh, RequiredBones );
					if ( bSuccess )
					{
						if (SkelMesh->GetSkeleton() != SelectedSkeleton)
						{
							SkelMesh->SetSkeleton(SelectedSkeleton);
							SkelMesh->MarkPackageDirty();
						}
						FAssetNotifications::SkeletonNeedsToBeSaved(SelectedSkeleton);
					}
					else
					{
						// if failed, ask if user would like to regenerate skeleton hierarchy
						if ( EAppReturnType::Yes == FMessageDialog::Open( EAppMsgType::YesNo, 
							LOCTEXT("SkeletonMergeBones_Override", "FAILED TO MERGE BONES:  \n\nThis could happen if significant hierarchical changes have been made,\ne.g. inserting a bone between nodes.\nWould you like to regenerate the skeleton from this mesh? \n\n***WARNING: THIS WILL INVALIDATE ALL ANIMATION DATA THAT IS LINKED TO THIS SKELETON***\n")))
						{
							if ( SelectedSkeleton->RecreateBoneTree( SkelMesh ) )
							{
								FAssetNotifications::SkeletonNeedsToBeSaved( SelectedSkeleton );
							}
						}
						else
						{
							FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("SkeletonMergeBonesFailure", "Failed to merge bones to Skeleton") );
						}
					}
				}
				else
				{
					FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("SkeletonMergeBonesFailure", "Failed to merge bones to Skeleton") );
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
