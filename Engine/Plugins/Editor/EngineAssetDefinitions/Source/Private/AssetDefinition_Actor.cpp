// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Actor)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_Actor::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (const FAssetData& Asset : OpenArgs.Assets)
	{
		if (AActor* Actor = Cast<AActor>(Asset.GetAsset()))
		{
			if (Actor->OpenAssetEditor())
			{
				return EAssetCommandResult::Handled;
			}
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
