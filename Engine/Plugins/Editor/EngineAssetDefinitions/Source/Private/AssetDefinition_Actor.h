// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_Actor.generated.h"

enum class EAssetCommandResult : uint8;
struct FAssetCategoryPath;
struct FAssetOpenArgs;

UCLASS()
class UAssetDefinition_Actor : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Actor", "Actor"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0,232,0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return AActor::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		return TConstArrayView<FAssetCategoryPath>();
	}
	virtual FText GetObjectDisplayNameText(UObject* Object) const override { return FText::FromString(CastChecked<AActor>(Object)->GetActorLabel()); }

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
