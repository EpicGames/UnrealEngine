// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFolder.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ActorFolder.generated.h"

struct FAssetCategoryPath;

UCLASS()
class UAssetDefinition_ActorFolder : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ActorFolder", "Actor Folder"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(182, 143, 85)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UActorFolder::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		return TConstArrayView<FAssetCategoryPath>();
	}
	virtual FText GetObjectDisplayNameText(UObject* Object) const override
	{
		UActorFolder* ActorFolder = CastChecked<UActorFolder>(Object);
		return FText::FromString(ActorFolder->GetDisplayName());
	}
	// UAssetDefinition End
};
