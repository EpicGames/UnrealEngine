// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FoliageType_Actor.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ActorFoliageSettings.generated.h"

UCLASS()
class UAssetDefinition_ActorFoliageSettings : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ActorFoliageSettings", "Actor Foliage"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(12, 65, 12)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UFoliageType_Actor::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Foliage };
		return Categories;
	}
	// UAssetDefinition End
};
