// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/CanvasRenderTarget2D.h"
#include "AssetDefinition_TextureRenderTarget2D.h"

#include "AssetDefinition_CanvasRenderTarget2D.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_CanvasRenderTarget2D : public UAssetDefinition_TextureRenderTarget2D
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CanvasRenderTarget2D", "Canvas Render Target"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCanvasRenderTarget2D::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture) };
		return Categories;
	}
	// UAssetDefinition End
};
