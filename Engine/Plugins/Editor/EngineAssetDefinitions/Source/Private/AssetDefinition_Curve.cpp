// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_Curve.h"

#include "ContentBrowserMenuContexts.h"
#include "EditorFramework/AssetImportData.h"
#include "CurveAssetEditorModule.h"
#include "DesktopPlatformModule.h"
#include "IAssetTools.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "ToolMenuSection.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Curve)

#define LOCTEXT_NAMESPACE "UAssetDefinition_Curve"

EAssetCommandResult UAssetDefinition_Curve::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FCurveAssetEditorModule& CurveAssetEditorModule = FModuleManager::LoadModuleChecked<FCurveAssetEditorModule>("CurveAssetEditor");

	for (UCurveBase* Curve : OpenArgs.LoadObjects<UCurveBase>())
	{
		TSharedRef<ICurveAssetEditor> NewCurveAssetEditor = CurveAssetEditorModule.CreateCurveAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Curve);
	}

	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_Curve::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UCurveBase* OldCurve = Cast<UCurveBase>(DiffArgs.OldAsset);
	const UCurveBase* NewCurve = Cast<UCurveBase>(DiffArgs.NewAsset);

	if (NewCurve == nullptr && OldCurve == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}

	// Build names for temp json files
	const auto AssetNameFallback = [OldCurve, NewCurve]()
	{
		if (OldCurve)
		{
			return OldCurve->GetName();
		}
		if (NewCurve)
		{
			return NewCurve->GetName();
		}
		return FString();
	};
	
	const FString OldAssetName = OldCurve ? OldCurve->GetName() : AssetNameFallback();
	const FString RelOldTempFileName = FString::Printf(TEXT("%sTemp%s-%s.json"), *FPaths::DiffDir(), *OldAssetName, *DiffArgs.OldRevision.Revision);
	const FString AbsoluteOldTempFileName = FPaths::ConvertRelativePathToFull(RelOldTempFileName);
	
	const FString NewAssetName = NewCurve ? NewCurve->GetName() : AssetNameFallback();
	const FString RelNewTempFileName = FString::Printf(TEXT("%sTemp%s-%s.json"), *FPaths::DiffDir(), *NewAssetName, *DiffArgs.NewRevision.Revision);
	const FString AbsoluteNewTempFileName = FPaths::ConvertRelativePathToFull(RelNewTempFileName);

	// save temp files
	const FString OldJson = OldCurve ? OldCurve->ExportAsJSONString() : TEXT("");
	const bool OldResult = FFileHelper::SaveStringToFile(OldJson, *AbsoluteOldTempFileName);
	const FString NewJson = NewCurve ? NewCurve->ExportAsJSONString() : TEXT("");
	const bool NewResult = FFileHelper::SaveStringToFile(NewJson, *AbsoluteNewTempFileName);

	if (OldResult && NewResult)
	{
		const FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;
		IAssetTools::Get().CreateDiffProcess(DiffCommand, AbsoluteOldTempFileName, AbsoluteNewTempFileName);

		return EAssetCommandResult::Handled;
	}

	return Super::PerformAssetDiff(DiffArgs);
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_CurveBase
{
	static void ExecuteImportFromJSON(const FToolMenuContext& InContext)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			for (UCurveBase* Curve : CBContext->LoadSelectedObjects<UCurveBase>())
			{
				const FText Title = FText::Format(LOCTEXT("Curve_ImportJSONDialogTitle", "Import '{0}' from JSON..."), FText::FromString(*Curve->GetName()));
				const FString CurrentFilename = Curve->AssetImportData->GetFirstFilename();
				const FString FileTypes = TEXT("Curve JSON (*.json)|*.json");

				TArray<FString> OutFilenames;
				DesktopPlatform->OpenFileDialog(
					ParentWindowWindowHandle,
					Title.ToString(),
					(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetPath(CurrentFilename),
					(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetBaseFilename(CurrentFilename) + TEXT(".json"),
					FileTypes,
					EFileDialogFlags::None,
					OutFilenames
					);

				if (OutFilenames.Num() > 0)
				{
					FString FileContent;
					FFileHelper::LoadFileToString(FileContent, *OutFilenames[0]);

					// If the JSON file contains the quoted string format, parse it into a valid JSON string, else use it as is
					FString JSONString;
					if (!FParse::QuotedString(*FileContent, JSONString))
					{
						JSONString = FileContent;
					}

					TArray<FString> Problems;
					Curve->ImportFromJSONString(JSONString, Problems);
					if (Problems.Num() > 0)
					{
						FString BigProblem;
						for (const FString& Problem :  Problems)
						{
							BigProblem.Append(Problem).Append(FString(TEXT("\n")));
						}
						FText WarningText = FText::FromString(FString::Format(TEXT("Couldn't import file {0} into {1}:\n\n{2}"), { OutFilenames[0], Curve->GetName(), BigProblem }));
						FMessageDialog::Open(EAppMsgType::Ok, WarningText);
					}
				}
			}
		}
	}

	static void ExecuteExportAsJSON(const FToolMenuContext& InContext)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			for (UCurveBase* Curve : CBContext->LoadSelectedObjects<UCurveBase>())
			{
				const FText Title = FText::Format(LOCTEXT("Curve_ExportJSONDialogTitle", "Export '{0}' as JSON..."), FText::FromString(*Curve->GetName()));
				const FString CurrentFilename = Curve->AssetImportData->GetFirstFilename();
				const FString FileTypes = TEXT("Curve JSON (*.json)|*.json");

				TArray<FString> OutFilenames;
				DesktopPlatform->SaveFileDialog(
					ParentWindowWindowHandle,
					Title.ToString(),
					(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetPath(CurrentFilename),
					(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetBaseFilename(CurrentFilename) + TEXT(".json"),
					FileTypes,
					EFileDialogFlags::None,
					OutFilenames
					);

				if (OutFilenames.Num() > 0)
				{
					FFileHelper::SaveStringToFile(Curve->ExportAsJSONString(), *OutFilenames[0]);
				}
			}
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UCurveBase::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
					{
						{
							const TAttribute<FText> Label = LOCTEXT("Curve_ImportFromJSON", "Import from JSON");
							const TAttribute<FText> ToolTip = LOCTEXT("Curve_ImportFromJSONTooltip", "Import the curve from a file containing JSON data.");
							const FSlateIcon Icon = FSlateIcon();
							const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteImportFromJSON);
							InSection.AddMenuEntry("Curve_ImportFromJSON", Label, ToolTip, Icon, UIAction);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("Curve_ExportAsJSON", "Export as JSON");
							const TAttribute<FText> ToolTip = LOCTEXT("Curve_ExportAsJSONTooltip", "Export the curve as a file containing JSON data.");
							const FSlateIcon Icon = FSlateIcon();
							const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteExportAsJSON);
							InSection.AddMenuEntry("Curve_ExportAsJSON", Label, ToolTip, Icon, UIAction);
						}
					}
				}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
