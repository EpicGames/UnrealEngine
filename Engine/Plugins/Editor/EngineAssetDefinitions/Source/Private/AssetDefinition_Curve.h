// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Curves/CurveBase.h"

#include "AssetDefinition_Curve.generated.h"

enum class EAssetCommandResult : uint8;
struct FAssetDiffArgs;
struct FAssetOpenArgs;

UCLASS()
class UAssetDefinition_Curve : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Curve", "Curve"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(78, 40, 165)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCurveBase::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "Curve", "Curve"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual bool CanImport() const override { return true; }

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	// UAssetDefinition End
};
