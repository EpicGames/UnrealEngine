// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition_Curve.h"
#include "Curves/CurveFloat.h"

#include "AssetDefinition_CurveFloat.generated.h"

UCLASS()
class UAssetDefinition_CurveFloat : public UAssetDefinition_Curve
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CurveFloat", "Float Curve"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCurveFloat::StaticClass(); }
	// UAssetDefinition End
};
