// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition_Curve.h"
#include "Curves/CurveLinearColor.h"

#include "AssetDefinition_CurveLinearColor.generated.h"

UCLASS()
class UAssetDefinition_CurveLinearColor : public UAssetDefinition_Curve
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CurveLinearColor", "Color Curve"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCurveLinearColor::StaticClass(); }
	// UAssetDefinition End
};
