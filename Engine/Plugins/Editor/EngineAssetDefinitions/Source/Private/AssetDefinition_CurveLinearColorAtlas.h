// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveLinearColorAtlas.h"
#include "AssetDefinition_Texture.h"

#include "AssetDefinition_CurveLinearColorAtlas.generated.h"

UCLASS()
class UAssetDefinition_CurveLinearColorAtlas : public UAssetDefinition_Texture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CurveLinearColorAtlas", "Curve Atlas"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,64,64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCurveLinearColorAtlas::StaticClass(); }
	virtual bool CanImport() const override { return false; }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "CurveLinearColorAtlas", "Curve"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
