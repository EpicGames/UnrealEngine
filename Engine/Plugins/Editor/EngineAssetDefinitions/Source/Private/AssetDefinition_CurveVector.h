// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition_Curve.h"
#include "Curves/CurveVector.h"

#include "AssetDefinition_CurveVector.generated.h"

UCLASS()
class UAssetDefinition_CurveVector : public UAssetDefinition_Curve
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CurveVector", "Vector Curve"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCurveVector::StaticClass(); }
	// UAssetDefinition End
};
