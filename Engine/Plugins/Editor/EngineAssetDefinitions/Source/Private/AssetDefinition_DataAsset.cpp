// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DataAsset.h"

#include "Editor.h"
#include "ClassViewerFilter.h"
#include "ContentBrowserMenuContexts.h"
#include "MergeUtils.h"
#include "ObjectTools.h"
#include "SDetailsDiff.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Engine/Engine.h"
#include "ToolMenuSection.h"
#include "HAL/PlatformFileManager.h"
#include "UObject/Linker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DataAsset)

#define LOCTEXT_NAMESPACE "UAssetDefinition_DataAsset"

namespace MenuExtension_DataAsset
{
	class FNewNodeClassFilter : public IClassViewerFilter
	{
	public:
		FNewNodeClassFilter(UClass* InBaseClass)
			: BaseClass(InBaseClass)
			, DisallowedClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_HideDropDown)
		{
		}

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			if (InClass != nullptr)
			{
				return InClass->IsChildOf(BaseClass) && !InClass->HasAnyClassFlags(DisallowedClassFlags);
			}
			return false;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return InUnloadedClassData->IsChildOf(BaseClass) && !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags);
		}

	private:
		UClass* BaseClass;

		EClassFlags DisallowedClassFlags;
	};

	bool IsChangeDataAssetClassVisible(const FToolMenuContext& MenuContext)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			for (const FContentBrowserItem& SelectedItem : Context->GetSelectedItems())
			{
				if (SelectedItem.CanEdit())
				{
					return true;
				}
			}
		}
		return false;
	}

	void ExecuteChangeDataAssetClass(const FToolMenuContext& MenuContext)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			const FText TitleText = LOCTEXT("DataAsset_PickNewDataAssetClass", "Pick New DataAsset Class");
			FClassViewerInitializationOptions Options;
			Options.ClassFilters.Add(MakeShared<FNewNodeClassFilter>(UDataAsset::StaticClass()));
			UClass* OutNewDataAssetClass = nullptr;
			const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, OutNewDataAssetClass, UDataAsset::StaticClass());

			if (bPressedOk && OutNewDataAssetClass != nullptr)
			{
				TSet<FName> EditableAssets;
				{
					const TArray<FContentBrowserItem>& SelectedItems = Context->GetSelectedItems();
					EditableAssets.Reserve(SelectedItems.Num());
					for (const FContentBrowserItem& SelectedItem : Context->GetSelectedItems())
					{
						if (SelectedItem.CanEdit())
						{
							EditableAssets.Add(SelectedItem.GetInternalPath());
						}
					}
				}
				ensure(!EditableAssets.IsEmpty());

				TArray<UDataAsset*> DataAssets = Context->LoadSelectedObjectsIf<UDataAsset>([&EditableAssets](const FAssetData& AssetData)
				{
					return EditableAssets.Contains(*AssetData.GetObjectPathString());
				});

				for (TWeakObjectPtr<UDataAsset> DataAssetPtr : DataAssets)
				{
					if (UDataAsset* OldDataAsset = DataAssetPtr.Get())
					{
						if (OldDataAsset && OldDataAsset->IsValidLowLevel())
						{
							FName ObjectName = OldDataAsset->GetFName();
							UObject* Outer = OldDataAsset->GetOuter();
							OldDataAsset->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors);

							UObject* NewDataAsset = NewObject<UObject>(Outer, OutNewDataAssetClass, ObjectName, OldDataAsset->GetFlags());

							// Migrate Data
							{
								UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyOptions;
								CopyOptions.bNotifyObjectReplacement = true;
								UEngine::CopyPropertiesForUnrelatedObjects(OldDataAsset, NewDataAsset, CopyOptions);
							}

							NewDataAsset->MarkPackageDirty();

							// Consolidate or "Replace" the old object with the new object for any living references.
							bool bShowDeleteConfirmation = false;
							TArray<UObject*> OldDataAssetArray({ (UObject*)OldDataAsset });
							ObjectTools::ConsolidateObjects(NewDataAsset, OldDataAssetArray, bShowDeleteConfirmation);
						}
					}
				}
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UDataAsset::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateStatic([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("DataAsset_ChangeClass", "Convert to Different DataAsset Type");
					const TAttribute<FText> ToolTip = LOCTEXT("DataAsset_ChangeClassTip", "Change the class these Data Assets are subclassed from.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.DataAsset");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteChangeDataAssetClass);
					UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&IsChangeDataAssetClassVisible);

					InSection.AddMenuEntry("DataAsset_ChangeClass", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}


FText UAssetDefinition_DataAsset::GetAssetDisplayName(const FAssetData& AssetData) const
{
	if (AssetData.IsValid())
	{
		const FAssetDataTagMapSharedView::FFindTagResult NativeClassTag = AssetData.TagsAndValues.FindTag(UDataAsset::NativeClassTag);
		if (NativeClassTag.IsSet())
		{
			if (UClass* FoundClass = FindObjectSafe<UClass>(nullptr, *NativeClassTag.GetValue()))
			{
				return FText::Format(LOCTEXT("DataAssetWithType", "Data Asset ({0})"), FoundClass->GetDisplayNameText());
			}
		}
	}

	return FText::GetEmpty();
}

EAssetCommandResult UAssetDefinition_DataAsset::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	if (DiffArgs.OldAsset == nullptr && DiffArgs.NewAsset == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}
	
	const TSharedRef<SDetailsDiff> DetailsDiff = SDetailsDiff::CreateDiffWindow(DiffArgs.OldAsset, DiffArgs.NewAsset, DiffArgs.OldRevision, DiffArgs.NewRevision, UDataAsset::StaticClass());
	// allow users to edit NewAsset if it's a local asset
	if (DiffArgs.NewAsset != nullptr && !FPackageName::IsTempPackage(DiffArgs.NewAsset->GetPackage()->GetName()))
	{
		DetailsDiff->SetOutputObject(DiffArgs.NewAsset);
	}
	return EAssetCommandResult::Handled;
}

bool UAssetDefinition_DataAsset::CanMerge() const
{
	return true;
}

EAssetCommandResult UAssetDefinition_DataAsset::Merge(const FAssetAutomaticMergeArgs& MergeArgs) const
{
	return MergeUtils::Merge(MergeArgs);
}

EAssetCommandResult UAssetDefinition_DataAsset::Merge(const FAssetManualMergeArgs& MergeArgs) const
{
	return MergeUtils::Merge(MergeArgs);
}


#undef LOCTEXT_NAMESPACE
