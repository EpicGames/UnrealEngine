// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "AssetDefinitionDefault.h"
#include "Misc/AssetCategoryPath.h"

#include "AssetDefinition_DataLayer.generated.h"

UCLASS()
class UAssetDefinition_DataLayer : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DataLayer", "Data Layer"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(12, 65, 12)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UDataLayerAsset::StaticClass(); }
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override { return FAssetSupportResponse::NotSupported(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::World, NSLOCTEXT("AssetDefinition", "DataLayer_SubMenu", "World"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
