// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_DataLayerInstance.generated.h"

struct FAssetCategoryPath;

UCLASS()
class UAssetDefinition_DataLayerInstance : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DataLayerInstance", "Data Layer Instance"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(52, 213, 235)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UDataLayerInstance::StaticClass(); }
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override { return FAssetSupportResponse::NotSupported(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		return TConstArrayView<FAssetCategoryPath>();
	}
	virtual FText GetObjectDisplayNameText(UObject* Object) const override
	{
		UDataLayerInstance* DataLayerInstance = CastChecked<UDataLayerInstance>(Object);
		return FText::FromString(DataLayerInstance->GetDataLayerShortName());
	}
	// UAssetDefinition End
};
