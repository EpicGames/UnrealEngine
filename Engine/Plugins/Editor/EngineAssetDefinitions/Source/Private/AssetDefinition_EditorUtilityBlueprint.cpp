// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_EditorUtilityBlueprint.h"

#include "ContentBrowserMenuContexts.h"

#include "Containers/ArrayView.h"
#include "Editor.h"
#include "EditorUtilityBlueprint.h"
#include "EditorUtilitySubsystem.h"
#include "IAssetTools.h"
#include "IBlutilityModule.h"
#include "ObjectEditorUtils.h"
#include "ToolMenus.h"
#include "Algo/RemoveIf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_EditorUtilityBlueprint)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText UAssetDefinition_EditorUtilityBlueprint::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_EditorUtilityBlueprintUpdate", "Editor Utility Blueprint");
}

FLinearColor UAssetDefinition_EditorUtilityBlueprint::GetAssetColor() const
{
	return FColor(0, 169, 255);
}

TSoftClassPtr<> UAssetDefinition_EditorUtilityBlueprint::GetAssetClass() const
{
	return UEditorUtilityBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_EditorUtilityBlueprint::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Blueprint, LOCTEXT("EditorUtilityBlueprint_SubMenus", "Editor Utilities"), ECategoryMenuType::Section) };
	return Categories;
}

namespace MenuExtension_EditorUtilityBlueprint
{
	using FWeakBlueprintPointerArray = TArray< TWeakObjectPtr<class UWidgetBlueprint> >;
	
	static void ExecuteEditorUtilityRun(const FToolMenuContext& InContext, TArray<UObject*> Objects)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();
			for (UObject* Object : Objects)
			{
				EditorUtilitySubsystem->TryRun(Object);
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit,[]
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UEditorUtilityBlueprint::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					UEditorUtilitySubsystem* EUSubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();

					// It would be ideal to not load selected object here, instead to defer the load to
					// ExecuteEditorUtilityRun(). Unfortunately, the GeneratedClass of a UBlueprint
					// is not accessible from AssetData without loading the asset. This is required for the CanRun()
					// check.  Results in an asset load on opening the context menu with these selected.
					TSet<FName> LoadTags;
					TArray<UObject*> SelectedAssets = Context->LoadSelectedObjects(LoadTags);

					const int32 EndOfRange = Algo::StableRemoveIf(SelectedAssets, [&EUSubsystem](UObject* Object)
					{
						return !EUSubsystem->CanRun(Object);
					});
					SelectedAssets.SetNum(EndOfRange);

					if (!SelectedAssets.IsEmpty())
					{
						const TAttribute<FText> Label = LOCTEXT("EditorUtility_Run", "Run Editor Utility Blueprint");
						const TAttribute<FText> ToolTip = LOCTEXT("EditorUtility_RunTooltip", "Runs this Editor Utility Blueprint.");
						const FSlateIcon Icon = FSlateIcon();
						
						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteEditorUtilityRun, MoveTemp(SelectedAssets));
						InSection.AddMenuEntry("EditorUtility_Run", Label, ToolTip, Icon, UIAction);
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
