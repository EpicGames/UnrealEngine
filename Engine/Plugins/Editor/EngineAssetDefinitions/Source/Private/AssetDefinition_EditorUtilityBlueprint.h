// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Script/AssetDefinition_Blueprint.h"

#include "AssetDefinition_EditorUtilityBlueprint.generated.h"

UCLASS()
class UAssetDefinition_EditorUtilityBlueprint : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
public:
	//~ UAssetDefinition_Blueprint Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	//~ UAssetDefinition_Blueprint End
	
};
