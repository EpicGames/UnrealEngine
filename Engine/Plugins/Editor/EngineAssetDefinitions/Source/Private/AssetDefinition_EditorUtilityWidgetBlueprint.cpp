// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_EditorUtilityWidgetBlueprint.h"

#include "Algo/RemoveIf.h"
#include "ContentBrowserMenuContexts.h"
#include "EditorUtilityLibrary.h"
#include "EditorUtilitySubsystem.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "IBlutilityModule.h"
#include "Misc/MessageDialog.h"
#include "SBlueprintDiff.h"
#include "ToolMenus.h"
#include "WidgetBlueprintEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_EditorUtilityWidgetBlueprint)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UAssetDefinition_EditorUtilityWidgetBlueprint::UAssetDefinition_EditorUtilityWidgetBlueprint() = default;

UAssetDefinition_EditorUtilityWidgetBlueprint::~UAssetDefinition_EditorUtilityWidgetBlueprint() = default;

FText UAssetDefinition_EditorUtilityWidgetBlueprint::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_EditorUtilityWidget", "Editor Utility Widget");
}

FLinearColor UAssetDefinition_EditorUtilityWidgetBlueprint::GetAssetColor() const
{
	return FColor(0, 169, 255);
}

TSoftClassPtr<> UAssetDefinition_EditorUtilityWidgetBlueprint::GetAssetClass() const
{
	return UEditorUtilityWidgetBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_EditorUtilityWidgetBlueprint::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Blueprint, LOCTEXT("EditorUtilityWidgetBlueprint_SubMenus", "Editor Utilities"), ECategoryMenuType::Section) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_EditorUtilityWidgetBlueprint::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	EToolkitMode::Type Mode = OpenArgs.GetToolkitMode();

	for (UBlueprint* Blueprint : OpenArgs.LoadObjects<UBlueprint>())
	{
		if (Blueprint && Blueprint->SkeletonGeneratedClass && Blueprint->GeneratedClass)
		{
			TSharedRef<FWidgetBlueprintEditor> NewBlueprintEditor(new FWidgetBlueprintEditor());

			const bool bShouldOpenInDefaultsMode = false;
			TArray<UBlueprint*> Blueprints;
			Blueprints.Add(Blueprint);

			NewBlueprintEditor->InitWidgetBlueprintEditor(EToolkitMode::Standalone, nullptr, Blueprints, bShouldOpenInDefaultsMode);
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FailedToLoadEditorUtilityWidgetBlueprint", "Editor Utility Widget could not be loaded because it derives from an invalid class.\nCheck to make sure the parent class for this blueprint hasn't been removed!"));	
		}
	}
	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_EditorUtilityWidgetBlueprint::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UBlueprint* OldBlueprint = Cast<UBlueprint>(DiffArgs.OldAsset);
	const UBlueprint* NewBlueprint = Cast<UBlueprint>(DiffArgs.NewAsset);
	SBlueprintDiff::CreateDiffWindow(OldBlueprint, NewBlueprint, DiffArgs.OldRevision, DiffArgs.NewRevision, GetAssetClass().Get());
	return EAssetCommandResult::Handled;
}

namespace MenuExtension_EditorUtilityWidgetBlueprint
{
	void ExecuteEditorUtilityEdit(const FToolMenuContext& InContext, TArray<FAssetData> AssetData)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			if (UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>())
			{
				TSet<FName> LoadTags;
				TArray<UWidgetBlueprint*> WidgetBlueprints = Context->LoadSelectedObjects<UWidgetBlueprint>(LoadTags);

				for (UWidgetBlueprint* WidgetBlueprint : WidgetBlueprints)
				{
					ensureMsgf(WidgetBlueprint->GeneratedClass, TEXT("Null generated class for WidgetBlueprint [%s]"), *WidgetBlueprint->GetName());
					if (WidgetBlueprint->GeneratedClass && WidgetBlueprint->GeneratedClass->IsChildOf(UEditorUtilityWidget::StaticClass()))
					{
						if (UEditorUtilityWidgetBlueprint* EditorWidget = Cast<UEditorUtilityWidgetBlueprint>(WidgetBlueprint))
						{
							EditorUtilitySubsystem->SpawnAndRegisterTab(EditorWidget);
						}
					}
				}
			}
		}
	}

	void ExecuteConvertToEditorUtilityBlueprint(const FToolMenuContext& InContext, TArray<FAssetData> AssetData)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			TArray<UWidgetBlueprint*> WidgetBlueprints = Context->LoadSelectedObjects<UWidgetBlueprint>();
			for(UWidgetBlueprint* WidgetBP : WidgetBlueprints)
			{
				UEditorUtilityLibrary::ConvertToEditorUtilityWidget(WidgetBP);
			}
		}
	}
	
	FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit,[]
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UEditorUtilityWidgetBlueprint::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					TArray<FAssetData> SelectedAssets(Context->SelectedAssets);
				
					const int32 EndOfRange = Algo::StableRemoveIf(SelectedAssets, [](const FAssetData& AssetData)
					{
						return !AssetData.IsInstanceOf(UWidgetBlueprint::StaticClass());
					});

					SelectedAssets.SetNum(EndOfRange);

					if (!SelectedAssets.IsEmpty())
					{
						const TAttribute<FText> Label = LOCTEXT("EditorUtilityWidget_Edit", "Run Editor Utility Widget");
						const TAttribute<FText> ToolTip = LOCTEXT("EditorUtilityWidget_EditTooltip", "Opens the tab built by this Editor Utility Widget Blueprint.");
						const FSlateIcon Icon = FSlateIcon();
						
						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteEditorUtilityEdit, MoveTemp(SelectedAssets));
						InSection.AddMenuEntry("EditorUtility_Run", Label, ToolTip, Icon, UIAction);
					}
				}
			}));

			UToolMenu* WigetBPMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.WidgetBlueprint.AssetActionsSubMenu");

			FToolMenuSection& WidgetBPSection = WigetBPMenu->FindOrAddSection("AssetContextAdvancedActions");
			WidgetBPSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
					{
						TArray<FAssetData> SelectedAssets(Context->SelectedAssets);

						const int32 EndOfRange = Algo::StableRemoveIf(SelectedAssets, [](const FAssetData& AssetData)
							{
								return !AssetData.IsInstanceOf(UWidgetBlueprint::StaticClass());
							});

						SelectedAssets.SetNum(EndOfRange);

						if (!SelectedAssets.IsEmpty())
						{
							const TAttribute<FText> Label = LOCTEXT("EditorUtilityWidget_Convert", "Convert to Editor Utility Widget");
							const TAttribute<FText> ToolTip = LOCTEXT("ConvertTooltip", "Convert this Widget Blueprint to an Editor Utility Widget");
							const FSlateIcon Icon = FSlateIcon();

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteConvertToEditorUtilityBlueprint, MoveTemp(SelectedAssets));
							InSection.AddMenuEntry("EditorUtility_Convert", Label, ToolTip, Icon, UIAction);
						}
					}
				}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
