// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_EditorUtilityWidgetBlueprint.generated.h"

UCLASS()
class UAssetDefinition_EditorUtilityWidgetBlueprint : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	UAssetDefinition_EditorUtilityWidgetBlueprint();
	virtual ~UAssetDefinition_EditorUtilityWidgetBlueprint() override;

	//~ UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	//~ UAssetDefinition End
	
};


