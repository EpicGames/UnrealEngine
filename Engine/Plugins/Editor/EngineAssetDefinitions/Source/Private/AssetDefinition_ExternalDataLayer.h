// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "AssetDefinitionDefault.h"
#include "AssetDefinition_DataLayer.h"

#include "AssetDefinition_ExternalDataLayer.generated.h"

UCLASS()
class UAssetDefinition_ExternalDataLayer : public UAssetDefinition_DataLayer
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ExternalDataLayer", "External Data Layer"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(UExternalDataLayerAsset::EditorUXColor); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UExternalDataLayerAsset::StaticClass(); }
	// UAssetDefinition End
};