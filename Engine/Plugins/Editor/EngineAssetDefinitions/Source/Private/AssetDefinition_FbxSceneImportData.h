// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/FbxSceneImportData.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_FbxSceneImportData.generated.h"

UCLASS()
class UAssetDefinition_FbxSceneImportData : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SceneImportData", "Fbx Scene Import Data"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255,0,255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UFbxSceneImportData::StaticClass(); }
	virtual bool CanImport() const override { return true; }
	// UAssetDefinition End
};
