// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_Font.h"

#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"
#include "EditorReimportHandler.h"
#include "FontEditorModule.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Font)

#define LOCTEXT_NAMESPACE "UAssetDefinition_Font"

EAssetCommandResult UAssetDefinition_Font::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	IFontEditorModule* FontEditorModule = &FModuleManager::LoadModuleChecked<IFontEditorModule>("FontEditor");
	
	for (UFont* Font : OpenArgs.LoadObjects<UFont>())
	{
		FontEditorModule->CreateFontEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Font);
	}
	
	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_Font
{
	void ExecuteReimport(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		for (UFont* Font : Context->LoadSelectedObjects<UFont>())
		{
			// Skip fonts that aren't using an offline cache, as they can't be reimported
			if (Font->FontCacheType == EFontCacheType::Offline)
			{
				// Fonts fail to reimport if they ask for a new file if missing
				FReimportManager::Instance()->Reimport(Font, /*bAskForNewFileIfMissing=*/false);
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UFont::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					// TODO NDarnell Having reimporting as a right click option for something like fonts is probably a
					// bit much, I think for something more complex that isn't always re-importable maybe they shouldn't
					// be right click options.  Though...I mean shouldn't even online fonts be reimportable?  They come
					// from OTF and TTF files...those things *can* be reloaded.
					// Also why do we even bother with this when we have IsImportable Asset()?
					// TODO Investigate, see if anyone remembers.
					
					const TAttribute<FText> Label = LOCTEXT("ReimportFontLabel", "Reimport");
					const TAttribute<FText> ToolTip = LOCTEXT("ReimportFontTooltip", "Reimport the selected font(s).");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.ReimportAsset");
					
					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteReimport);
					InSection.AddMenuEntry("ReimportFont", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
