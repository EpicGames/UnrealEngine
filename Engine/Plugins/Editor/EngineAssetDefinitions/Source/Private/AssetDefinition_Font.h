// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Font.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_Font.generated.h"

UCLASS()
class UAssetDefinition_Font : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Font", "Font"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,128,64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UFont::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::UI, NSLOCTEXT("AssetDefinition", "Font_SubMenu", "Font"), ECategoryMenuType::Section) };
		return Categories;
	}
	
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
