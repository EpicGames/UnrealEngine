// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/FontFace.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_FontFace.generated.h"

UCLASS()
class UAssetDefinition_FontFace : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_FontFace", "Font Face"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(184,184,112)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UFontFace::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::UI };
		return Categories;
	}
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const
	{
		return FAssetSupportResponse::NotSupported();
	}
	
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
	
private:
	/** Can we execute a reimport for the selected objects? */
	bool CanExecuteReimport(const TArray<TWeakObjectPtr<UFontFace>> Objects) const;

	/** Handler for when Reimport is selected */
	void ExecuteReimport(const TArray<TWeakObjectPtr<UFontFace>> Objects) const;
};
