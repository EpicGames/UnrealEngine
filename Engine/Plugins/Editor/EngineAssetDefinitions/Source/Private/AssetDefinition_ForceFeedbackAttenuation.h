// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/ForceFeedbackAttenuation.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ForceFeedbackAttenuation.generated.h"

UCLASS()
class UAssetDefinition_ForceFeedbackAttenuation : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ForceFeedbackAttenuation", "Force Feedback Attenuation"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(175, 0, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UForceFeedbackAttenuation::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
   	{
   		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Input, NSLOCTEXT("AssetDefinition", "ForceFeedbackAttenuation_SubMenu", "Force Feedback"), ECategoryMenuType::Section) };
   		return Categories;
   	}
	// UAssetDefinition End
};
