// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ForceFeedbackEffect.h"

#include "AssetViewUtils.h"
#include "ContentBrowserMenuContexts.h"
#include "Framework/Application/SlateApplication.h"
#include "ObjectEditorUtils.h"
#include "ToolMenus.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ForceFeedbackEffect)

#define LOCTEXT_NAMESPACE "AssetDefinition_ForceFeedbackEffect"

FText UAssetDefinition_ForceFeedbackEffect::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_ForceFeedbackEffect", "Force Feedback Effect");
}

FLinearColor UAssetDefinition_ForceFeedbackEffect::GetAssetColor() const
{
	return FColor(175, 0, 0);
}

TSoftClassPtr<UObject> UAssetDefinition_ForceFeedbackEffect::GetAssetClass() const
{
	return UForceFeedbackEffect::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ForceFeedbackEffect::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Input, NSLOCTEXT("AssetDefinition", "ForceFeedbackEffect_SubMenu", "Force Feedback"), ECategoryMenuType::Section) };
	return Categories;
}

// Menu Extensions
// --------------------------------------------------------------------

namespace MenuExtension_ForceFeedbackEffect
{
	bool IsEffectPlaying(const TArray<TWeakObjectPtr<UForceFeedbackEffect>>& Objects)
	{
		if (UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ForceFeedbackEffect)
		{
			for (const TWeakObjectPtr<UForceFeedbackEffect>& EffectPtr : Objects)
			{
				UForceFeedbackEffect* Effect = EffectPtr.Get();
				if (Effect && UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ForceFeedbackEffect == Effect)
				{
					return true;
				}
			}
		}

		return false;
	}

	bool IsEffectPlaying(const UForceFeedbackEffect* ForceFeedbackEffect)
	{
		return UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ForceFeedbackEffect && UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ForceFeedbackEffect == ForceFeedbackEffect;
	}

	bool IsEffectPlaying(const FAssetData& AssetData)
	{
		if (UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ForceFeedbackEffect)
		{
			if (UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ForceFeedbackEffect->GetFName() == AssetData.AssetName)
			{
				if (UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ForceFeedbackEffect->GetOutermost()->GetFName() == AssetData.PackageName)
				{
					return true;
				}
			}
		}

		return false;
	}

	void StopEffect() 
	{
		UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ResetDeviceProperties();
		UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ForceFeedbackEffect = nullptr;

		if (IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface())
		{
			InputInterface->SetForceFeedbackChannelValues(0, FForceFeedbackValues());
		}
	}

	void PlayEffect(UForceFeedbackEffect* Effect)
	{
		if (Effect)
		{
			UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ForceFeedbackEffect = Effect;
			UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().PlayTime = 0.f;
			UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().PlatformUser = IPlatformInputDeviceMapper::Get().GetPrimaryPlatformUser();
			UAssetDefinition_ForceFeedbackEffect::GetPreviewForceFeedbackEffect().ActivateDeviceProperties();
		}
		else
		{
			StopEffect();
		}
	}

	bool CanExecutePlayCommand(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		return Context->SelectedAssets.Num() == 1;
	}

	void ExecutePlayEffect(const TArray<TWeakObjectPtr<UForceFeedbackEffect>>& Objects)
	{
		for (const TWeakObjectPtr<UForceFeedbackEffect>& EffectPtr : Objects)
		{
			if (UForceFeedbackEffect* Effect = EffectPtr.Get())
			{
				// Only play the first valid effect
				PlayEffect(Effect);
				break;
			}
		}
	}

	void ExecutePlayEffect(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		TArray<UObject*> SelectedForceFeedbackEffectObjects;
		AssetViewUtils::FLoadAssetsSettings Settings{
			// Default settings
		};

		AssetViewUtils::LoadAssetsIfNeeded(Context->SelectedAssets, SelectedForceFeedbackEffectObjects, Settings);
		const TArray<TWeakObjectPtr<UForceFeedbackEffect>>& Objects = FObjectEditorUtils::GetTypedWeakObjectPtrs<UForceFeedbackEffect>(SelectedForceFeedbackEffectObjects);
		ExecutePlayEffect(Objects);
	}

	void ExecuteStopEffect()
	{
		StopEffect();
	}

	void ExecuteStopEffect(const FToolMenuContext& InContext)
	{
		ExecuteStopEffect();
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UForceFeedbackEffect::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					const TAttribute<FText> PlayLabel = LOCTEXT("ForceFeedbackEffect_PlayEffect", "Play");
					const TAttribute<FText> PlayToolTip = LOCTEXT("ForceFeedbackEffect_PlayEffectTooltip", "Plays the selected force feedback effect.");
					const FSlateIcon PlayIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetAction.PlayIcon");

					FToolUIAction PlayUIAction;
					PlayUIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecutePlayEffect);
					PlayUIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecutePlayCommand);

					InSection.AddMenuEntry(TEXT("ForceFeedbackEffect_PlayEffect"), PlayLabel, PlayToolTip, PlayIcon, PlayUIAction);

					const TAttribute<FText> StopLabel = LOCTEXT("ForceFeedbackEffect_StopEffect", "Stop");
					const TAttribute<FText> StopToolTip = LOCTEXT("ForceFeedbackEffect_StopEffectTooltip", "Stops the selected force feedback effect.");
					const FSlateIcon StopIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetAction.StopIcon");

					FToolUIAction StopUIAction;
					StopUIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteStopEffect);

					InSection.AddMenuEntry(TEXT("ForceFeedbackEffect_StopEffect"), StopLabel, StopToolTip, StopIcon, StopUIAction);
				}
			}));
		}));
	});
}

// --------------------------------------------------------------------
// Menu Extensions

EAssetCommandResult UAssetDefinition_ForceFeedbackEffect::ActivateAssets(const FAssetActivateArgs& InActivateArgs) const
{
	if (InActivateArgs.ActivationMethod == EAssetActivationMethod::Previewed)
	{
		TArray<UObject*> Objects = InActivateArgs.LoadObjects<UObject>();
		for (UObject* Object : Objects)
		{
			if (UForceFeedbackEffect* TargetEffect = Cast<UForceFeedbackEffect>(Object))
			{
				// Only target the first valid effect
				TArray<TWeakObjectPtr<UForceFeedbackEffect>> EffectList;
				EffectList.Add(MakeWeakObjectPtr(TargetEffect));
				if (MenuExtension_ForceFeedbackEffect::IsEffectPlaying(EffectList))
				{
					MenuExtension_ForceFeedbackEffect::ExecuteStopEffect();
				}
				else
				{
					MenuExtension_ForceFeedbackEffect::ExecutePlayEffect(EffectList);
				}

				return EAssetCommandResult::Handled;
			}
		}
	}
	return EAssetCommandResult::Unhandled;
}

TSharedPtr<SWidget> UAssetDefinition_ForceFeedbackEffect::GetThumbnailOverlay(const FAssetData& InAssetData) const
{
	auto OnGetDisplayBrushLambda = [this, InAssetData]() -> const FSlateBrush*
	{
		if (MenuExtension_ForceFeedbackEffect::IsEffectPlaying(InAssetData))
		{
			return FAppStyle::GetBrush("MediaAsset.AssetActions.Stop.Large");
		}
		return FAppStyle::GetBrush("MediaAsset.AssetActions.Play.Large");
	};

	auto OnClickedLambda = [InAssetData]() -> FReply
	{
		if (MenuExtension_ForceFeedbackEffect::IsEffectPlaying(InAssetData))
		{
			MenuExtension_ForceFeedbackEffect::StopEffect();
		}
		else
		{
			// Load and play asset
			MenuExtension_ForceFeedbackEffect::PlayEffect(Cast<UForceFeedbackEffect>(InAssetData.GetAsset()));
		}
		return FReply::Handled();
	};

	auto OnToolTipTextLambda = [this, InAssetData]() -> FText
	{
		if (MenuExtension_ForceFeedbackEffect::IsEffectPlaying(InAssetData))
		{
			return LOCTEXT("Thumbnail_StopForceFeedbackToolTip", "Stop selected force feedback effect");
		}
		return LOCTEXT("Thumbnail_PlayForceFeedbackToolTip", "Play selected force feedback effect");
	};

	TSharedRef<SBox> Box = SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2));

	auto OnGetVisibilityLambda = [this, Box, InAssetData]() -> EVisibility
	{
		if (Box->IsHovered() || MenuExtension_ForceFeedbackEffect::IsEffectPlaying(InAssetData))
		{
			return EVisibility::Visible;
		}
		return EVisibility::Hidden;
	};

	TSharedRef<SButton> BoxContent = SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ToolTipText_Lambda(OnToolTipTextLambda)
		.Cursor(EMouseCursor::Default) // The outer widget can specify a DragHand cursor, so we need to override that here
		.ForegroundColor(FSlateColor::UseForeground())
		.IsFocusable(false)
		.OnClicked_Lambda(OnClickedLambda)
		.Visibility_Lambda(OnGetVisibilityLambda)
		[
			SNew(SBox)
			.MinDesiredWidth(16.f)
			.MinDesiredHeight(16.f)
			[
				SNew(SImage)
				.Image_Lambda(OnGetDisplayBrushLambda)
			]
		];

	Box->SetContent(BoxContent);
	Box->SetVisibility(EVisibility::Visible);

	return Box;
}

bool UAssetDefinition_ForceFeedbackEffect::GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const
{
	OutActionOverlayInfo.IsActionPlayingDelegate = FIsActionPlaying::CreateLambda([InAssetData] ()
		{
			return MenuExtension_ForceFeedbackEffect::IsEffectPlaying(InAssetData);
		});

	auto OnClickedLambda = [InAssetData]() -> FReply
	{
		if (MenuExtension_ForceFeedbackEffect::IsEffectPlaying(InAssetData))
		{
			MenuExtension_ForceFeedbackEffect::StopEffect();
		}
		else
		{
			// Load and play asset
			MenuExtension_ForceFeedbackEffect::PlayEffect(Cast<UForceFeedbackEffect>(InAssetData.GetAsset()));
		}
		return FReply::Handled();
	};

	auto OnToolTipTextLambda = [InAssetData]() -> FText
	{
		if (MenuExtension_ForceFeedbackEffect::IsEffectPlaying(InAssetData))
		{
			return LOCTEXT("Thumbnail_StopForceFeedbackToolTip", "Stop selected force feedback effect");
		}
		return LOCTEXT("Thumbnail_PlayForceFeedbackToolTip", "Play selected force feedback effect");
	};

	OutActionOverlayInfo.ActionButtonArgs = SButton::FArguments()
		.ToolTipText_Lambda(OnToolTipTextLambda)
		.OnClicked_Lambda(OnClickedLambda);

	return true;
}

bool FPreviewForceFeedbackEffect::IsTickable() const
{
	return (ForceFeedbackEffect != nullptr);
}

void FPreviewForceFeedbackEffect::Tick( float DeltaTime )
{
	FForceFeedbackValues ForceFeedbackValues;

	if (!Update(DeltaTime, ForceFeedbackValues))
	{
		ResetDeviceProperties();
		ForceFeedbackEffect = nullptr;
	}

	IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
	if (InputInterface)
	{
		InputInterface->SetForceFeedbackChannelValues(0, ForceFeedbackValues);
	}
}

TStatId FPreviewForceFeedbackEffect::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FPreviewForceFeedbackEffect, STATGROUP_Tickables);
}

void FPreviewForceFeedbackEffect::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(ForceFeedbackEffect);
}

#undef LOCTEXT_NAMESPACE
