// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "GameFramework/ForceFeedbackEffect.h"
#include "GenericPlatform/IInputInterface.h"
#include "TickableEditorObject.h"
#include "UObject/GCObject.h"
#include "AssetDefinition_ForceFeedbackEffect.generated.h"

struct FPreviewForceFeedbackEffect : public FActiveForceFeedbackEffect, public FTickableEditorObject, public FGCObject
{
	// FTickableEditorObject Implementation
	virtual bool IsTickable() const override;
	virtual void Tick( float DeltaTime ) override;
	virtual TStatId GetStatId() const override;

	// FGCObject Implementation
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FPreviewForceFeedbackEffect");
	}
};

UCLASS()
class UAssetDefinition_ForceFeedbackEffect : public UAssetDefinitionDefault
{
	GENERATED_BODY()

protected:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& InActivateArgs) const override;
	virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& InAssetData) const override;
	virtual bool GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const override;
	// UAssetDefinition End

public:

	static FPreviewForceFeedbackEffect& GetPreviewForceFeedbackEffect()
	{
		static FPreviewForceFeedbackEffect Instance;
		return Instance;
	}
};
