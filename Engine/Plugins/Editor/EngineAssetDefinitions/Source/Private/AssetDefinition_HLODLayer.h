// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/HLODLayer.h"
#include "AssetDefinitionDefault.h"
#include "Misc/AssetCategoryPath.h"

#include "AssetDefinition_HLODLayer.generated.h"

UCLASS()
class UAssetDefinition_HLODLayer : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_HLODLayer", "HLOD Layer"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 200, 200)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UHLODLayer::StaticClass(); }
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override { return FAssetSupportResponse::NotSupported(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::World, NSLOCTEXT("AssetDefinition", "HLODLayer_SubMenu", "World"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
