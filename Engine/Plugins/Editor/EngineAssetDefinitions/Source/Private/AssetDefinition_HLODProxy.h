// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/HLODProxy.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_HLODProxy.generated.h"

UCLASS()
class UAssetDefinition_HLODProxy : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_HLODProxy", "HLOD Proxy"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 200, 200)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UHLODProxy::StaticClass(); }
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override { return FAssetSupportResponse::NotSupported(); }
	// UAssetDefinition End
};
