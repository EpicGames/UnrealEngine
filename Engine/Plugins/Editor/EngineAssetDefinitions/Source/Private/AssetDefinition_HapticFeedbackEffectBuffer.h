// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Haptics/HapticFeedbackEffect_Buffer.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_HapticFeedbackEffectBuffer.generated.h"

UCLASS()
class UAssetDefinition_HapticFeedbackEffectBuffer : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "FAssetTypeActions_HapticFeedbackEffectBuffer", "Haptic Feedback Buffer"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(175, 0, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UHapticFeedbackEffect_Buffer::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Input, NSLOCTEXT("AssetDefinition", "HapticFeedbackEffectBuffer_SubMenu", "Haptic Feedback"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
