// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Haptics/HapticFeedbackEffect_Curve.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_HapticFeedbackEffectCurve.generated.h"

UCLASS()
class UAssetDefinition_HapticFeedbackEffectCurve : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_HapticFeedbackEffectCurve", "Haptic Feedback Curve"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(175, 0, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UHapticFeedbackEffect_Curve::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Input, NSLOCTEXT("AssetDefinition", "HapticFeedbackEffectCurve_SubMenu", "Haptic Feedback"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
