// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Haptics/HapticFeedbackEffect_SoundWave.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_HapticFeedbackEffectSoundWave.generated.h"

UCLASS()
class UAssetDefinition_HapticFeedbackEffectSoundWave : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "FAssetTypeActions_HapticFeedbackEffectSoundWave", "Haptic Feedback Sound Wave"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(175, 0, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UHapticFeedbackEffect_SoundWave::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Input, NSLOCTEXT("AssetDefinition", "HapticFeedbackEffectSoundWave_SubMenu", "Haptic Feedback"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
