// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "FoliageType_InstancedStaticMesh.h"

#include "AssetDefinition_InstancedFoliageSettings.generated.h"

UCLASS()
class UAssetDefinition_InstancedFoliageSettings : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_InstancedFoliageSettings", "Static Mesh Foliage"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(12, 173, 12)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UFoliageType_InstancedStaticMesh::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Foliage };
		return Categories;
	}
	// UAssetDefinition End
};
