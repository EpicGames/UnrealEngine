// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LandscapeGrassType.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_LandscapeGrassType.generated.h"

UCLASS()
class UAssetDefinition_LandscapeGrassType : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_LandscapeGrassType", "Landscape Grass Type"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,255,128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return ULandscapeGrassType::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Foliage };
		return Categories;
	}
	// UAssetDefinition End
};
