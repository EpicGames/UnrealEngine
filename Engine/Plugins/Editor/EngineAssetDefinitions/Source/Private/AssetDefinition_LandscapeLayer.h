// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LandscapeLayerInfoObject.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_LandscapeLayer.generated.h"

UCLASS()
class UAssetDefinition_LandscapeLayer : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_LandscapeLayer", "Landscape Layer"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,192,255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return ULandscapeLayerInfoObject::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Misc, NSLOCTEXT("AssetDefinition", "LandscapeLayer_SubMenu", "Other"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	// UAssetDefinition End
};
