// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/LightWeightInstanceManager.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_LightWeightInstance.generated.h"

UCLASS(Deprecated, meta = (DeprecationMessage ="Deprecated in 5.8. Consider using InstancedActors for similar functionality"))
class UDEPRECATED_AssetDefinition_LightWeightInstance : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_LightWeightInstance", "Light Weight Instance"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 105, 180)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return ADEPRECATED_LightWeightInstanceManager::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Gameplay, NSLOCTEXT("AssetDefinition", "LightWeightInstance_SubMenu", "Other")) };
		return Categories;
	}
	// UAssetDefinition End
};
