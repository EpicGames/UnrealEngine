// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/NodeMappingContainer.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_NodeMappingContainer.generated.h"

UCLASS()
class UAssetDefinition_NodeMappingContainer : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_NodeMappingContainer", "Node Mapping Container"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(112,146,190)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UNodeMappingContainer::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Animation };
		return Categories;
	}
	// UAssetDefinition End
};
