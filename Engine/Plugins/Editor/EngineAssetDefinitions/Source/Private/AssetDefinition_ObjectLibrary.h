// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/ObjectLibrary.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ObjectLibrary.generated.h"

UCLASS()
class UAssetDefinition_ObjectLibrary : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ObjectLibrary", "Object Library"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 63, 108)); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "ObjectLibrary", "Other"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UObjectLibrary::StaticClass(); }
	// UAssetDefinition End
};
