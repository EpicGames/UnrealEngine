// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicalMaterials/PhysicalMaterial.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_PhysicalMaterial.generated.h"

UCLASS()
class UAssetDefinition_PhysicalMaterial : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override
	{
		return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_PhysicalMaterial", "Physical Material");
	}

	virtual FLinearColor GetAssetColor() const override
	{
		return FLinearColor(FColor(200,192,128));
	}

	virtual TSoftClassPtr<UObject> GetAssetClass() const override
	{
		return UPhysicalMaterial::StaticClass();
	}

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Physics, NSLOCTEXT("AssetDefinition", "PhysicalMaterial_SubMenu", "Physical Material"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	// UAssetDefinition End
};
