// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PhysicalMaterialMask.h"
#include "ContentBrowserMenuContexts.h"
#include "EditorFramework/AssetImportData.h"
#include "PhysicalMaterialMaskImport.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_PhysicalMaterialMask"

namespace MenuExtension::PhysicalMaterialMask
{
	void GetResolvedSourceFilePaths(const UPhysicalMaterialMask* InMaterialMask, TArray<FString>& OutSourceFilePaths)
	{
		if (!InMaterialMask)
		{
			OutSourceFilePaths.Emplace(FString());
			return;
		}

		TArray<FString> ExtractedFilenames;
		if (UAssetImportData* AssetImportData = InMaterialMask->AssetImportData)
		{
			if (AssetImportData->GetSourceFileCount() > 0)
			{
				if (!AssetImportData->GetSourceData().SourceFiles[0].RelativeFilename.IsEmpty())
				{
					InMaterialMask->AssetImportData->ExtractFilenames(ExtractedFilenames);
				}
			}
		}

		if (ExtractedFilenames.Num() > 0)
		{
			OutSourceFilePaths.Emplace(ExtractedFilenames[0]);
		}
		else
		{
			OutSourceFilePaths.Emplace(FString());
		}
	}

	bool IsOneAssetSelected(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return false;
		}

		TArray<FAssetData> AssetData = CBContext->SelectedAssets;
		return AssetData.Num() == 1;
	}

	void ExecuteImport(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		if (CBContext == nullptr)
		{
			return;
		}

		TArray<FAssetData> AssetData = CBContext->SelectedAssets;
		if (AssetData.Num() != 1)
		{
			return;
		}

		TArray<FString> OutResolvedFilePaths;
		UPhysicalMaterialMask* SelectedMask = CBContext->LoadFirstSelectedObject<UPhysicalMaterialMask>();
		GetResolvedSourceFilePaths(SelectedMask, OutResolvedFilePaths);

		if (SelectedMask && OutResolvedFilePaths.Num() == 1 && OutResolvedFilePaths[0].IsEmpty())
		{
			FPhysicalMaterialMaskImport::ImportMaskTexture(SelectedMask);
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			// Physical Material Mask Action Registration
			{
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UPhysicalMaterialMask::StaticClass());

				FToolMenuSection& PhysicalMaterialMaskSection = Menu->FindOrAddSection("GetAssetActions");
				PhysicalMaterialMaskSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
						{
							if (Context->SelectedAssets.Num() > 0)
							{
								const FAssetData& AssetData = Context->SelectedAssets[0];
								if (AssetData.AssetClassPath == UPhysicalMaterialMask::StaticClass()->GetClassPathName())
								{
									// Import Mask Texture
									{
										const TAttribute<FText> Label = LOCTEXT("PhysicalMaterialMask_ImportMaskTextureLabel", "Import Mask Texture");
										const TAttribute<FText> ToolTip = LOCTEXT("PhysicalMaterialMask_ImportMaskTextureTooltip", "Imports a texture to use as a physical material mask.");
										const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Material");

										FToolUIAction UIAction;
										UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteImport);
										UIAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateStatic(&IsOneAssetSelected);
										InSection.AddMenuEntry("PhysicalMaterialMask_Import", Label, ToolTip, Icon, UIAction);
									}
								}
							}
						}
					}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
