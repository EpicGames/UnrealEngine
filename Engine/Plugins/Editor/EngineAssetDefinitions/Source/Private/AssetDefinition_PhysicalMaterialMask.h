// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "PhysicalMaterials/PhysicalMaterialMask.h"
#include "AssetDefinition_PhysicalMaterialMask.generated.h"

UCLASS()
class UAssetDefinition_PhysicalMaterialMask : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override
	{
		return NSLOCTEXT("AssetDefinition", "AssetDefinition_PhysicalMaterialMask", "Physical Material Mask");
	}

	virtual FLinearColor GetAssetColor() const override
	{
		return FColor(200,162,108);
	}

	virtual TSoftClassPtr<UObject> GetAssetClass() const override
	{
		return UPhysicalMaterialMask::StaticClass();
	}

	virtual bool CanImport() const override
	{
		return true;
	}

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Physics, NSLOCTEXT("AssetDefinition", "AssetDefinition_PhysicalMaterialMaskSubMenu", "Physical Material"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	// UAssetDefinition End
};
