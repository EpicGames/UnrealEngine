// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PhysicsAsset.h"
#include "Modules/ModuleManager.h"
#include "PhysicsAssetEditorModule.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "IPhysicsAssetEditor.h"
#include "Features/IModularFeatures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_PhysicsAsset)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UThumbnailInfo* UAssetDefinition_PhysicsAsset::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), USceneThumbnailInfo::StaticClass());
}

EAssetCommandResult UAssetDefinition_PhysicsAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<IPhysicsAssetEditorOverride*> ExternalEditors = IModularFeatures::Get().GetModularFeatureImplementations<IPhysicsAssetEditorOverride>(IPhysicsAssetEditorOverride::ModularFeatureName);

	for (UPhysicsAsset* PhysicsAsset : OpenArgs.LoadObjects<UPhysicsAsset>())
	{
		bool bHandledExternally = false;

		for(IPhysicsAssetEditorOverride* OverrideEditor : ExternalEditors)
		{
			if(OverrideEditor->OpenAsset(PhysicsAsset))
			{
				bHandledExternally = true;
			}
		}

		if(bHandledExternally)
		{
			continue;
		}

		IPhysicsAssetEditorModule* PhysicsAssetEditorModule = &FModuleManager::LoadModuleChecked<IPhysicsAssetEditorModule>("PhysicsAssetEditor");
		PhysicsAssetEditorModule->CreatePhysicsAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, PhysicsAsset);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
