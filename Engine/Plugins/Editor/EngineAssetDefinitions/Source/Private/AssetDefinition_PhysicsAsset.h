// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/PhysicsAsset.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_PhysicsAsset.generated.h"

UCLASS()
class UAssetDefinition_PhysicsAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override
	{
		return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_PhysicsAsset", "Physics Asset");
	}

	virtual FLinearColor GetAssetColor() const override
	{
		return FLinearColor(FColor(255,192,128));
	}

	virtual TSoftClassPtr<UObject> GetAssetClass() const override
	{
		return UPhysicsAsset::StaticClass();
	}

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Physics, NSLOCTEXT("AssetDefinition", "PhysicalMaterial_SubMenu", "Physical Material"), ECategoryMenuType::Section)
			};
		return Categories;
	}

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAsset) const override;
	// UAssetDefinition End
};
