// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Engine/PreviewMeshCollection.h"

#include "AssetDefinition_PreviewMeshCollection.generated.h"

UCLASS()
class UAssetDefinition_PreviewMeshCollection : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_PreviewMeshCollection", "Preview Mesh Collection"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(190, 20, 70)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPreviewMeshCollection::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "PreviewMeshCollection_SubMenu", "Other"), ECategoryMenuType::Section) };
		return Categories;
	}
};
