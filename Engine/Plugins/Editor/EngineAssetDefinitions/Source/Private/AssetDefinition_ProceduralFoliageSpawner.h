// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProceduralFoliageSpawner.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ProceduralFoliageSpawner.generated.h"

UCLASS()
class UAssetDefinition_ProceduralFoliageSpawner : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ProceduralFoliageSpawner", "Procedural Foliage Spawner"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(7, 103, 7)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UProceduralFoliageSpawner::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Foliage };
		return Categories;
	}
	//bool FAssetTypeActions_ProceduralFoliageSpawner::CanFilter()
	//{
	//	return GetDefault<UEditorExperimentalSettings>()->bProceduralFoliage;
	//}
	// UAssetDefinition End
};
