// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_Redirector.h"

#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "ContentBrowserMenuContexts.h"
#include "IAssetTools.h"
#include "ToolMenus.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Redirector)

#define LOCTEXT_NAMESPACE "UAssetDefinition_Redirector"

namespace RedirectorUtilities
{
	static void FindTargets(const TArray<UObjectRedirector*>& Redirectors)
	{
		TArray<UObject*> ObjectsToSync;
		for (auto RedirectorIt = Redirectors.CreateConstIterator(); RedirectorIt; ++RedirectorIt)
		{
			const UObjectRedirector* Redirector = *RedirectorIt;
			if ( Redirector && Redirector->DestinationObject )
			{
				ObjectsToSync.Add(Redirector->DestinationObject);
			}
		}

		if ( ObjectsToSync.Num() > 0 )
		{
			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
			AssetTools.SyncBrowserToAssets(ObjectsToSync);
		}
	}
}

namespace MenuExtension_Redirector
{
	static void ExecuteFindTarget(const FToolMenuContext& MenuContext)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			TArray<UObjectRedirector*> Redirectors = Context->LoadSelectedObjects<UObjectRedirector>();
			RedirectorUtilities::FindTargets(Redirectors);
		}
	}

	static void ExecuteFixUp(const FToolMenuContext& MenuContext)
	{
		// This will fix references to selected redirectors, except in the following cases:
		// Redirectors referenced by unloaded maps will not be fixed up, but any references to it that can be fixed up will
		// Redirectors referenced by code will not be completely fixed up
		// Redirectors that are not at head revision or checked out by another user will not be completely fixed up
		// Redirectors whose referencers are not at head revision, are checked out by another user, or are refused to be checked out will not be completely fixed up.

		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			FScopedSlowTask SlowTask(3, LOCTEXT("FixupRedirectorsSlowTask", "Fixing up redirectors"));
			SlowTask.MakeDialog(true);

			SlowTask.EnterProgressFrame(1, LOCTEXT("FixupRedirectors_LoadAssets", "Loading Assets..."));
			TArray<UObject*> Objects;
			AssetViewUtils::FLoadAssetsSettings Settings{
				.bFollowRedirectors = false,
				.bAllowCancel = true,
			};
			AssetViewUtils::ELoadAssetsResult Result = AssetViewUtils::LoadAssetsIfNeeded(Context->SelectedAssets, Objects, Settings);
			if (Result != AssetViewUtils::ELoadAssetsResult::Cancelled && !SlowTask.ShouldCancel())
			{
				TArray<UObjectRedirector*> Redirectors;
				for (UObject* Object : Objects)
				{
					if (UObjectRedirector* Redirector = Cast<UObjectRedirector>(Object))
					{
						Redirectors.Add(Redirector);
					}
				}

				if (Redirectors.Num() > 0)
				{
					SlowTask.EnterProgressFrame(1, LOCTEXT("FixupRedirectors_FixupReferencers", "Fixing up referencers..."));
					IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
					AssetTools.FixupReferencers(Redirectors, /*bCheckoutDialogPrompt=*/true, ERedirectFixupMode::PromptForDeletingRedirectors);
				}
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UObjectRedirector::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("Redirector_FindTarget", "Find Target");
					const TAttribute<FText> ToolTip = LOCTEXT("Redirector_FindTargetTooltip", "Finds the asset that this redirector targets in the asset tree.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.ObjectRedirector");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindTarget);

					InSection.AddMenuEntry("Redirector_FindTarget", Label, ToolTip, Icon, UIAction);
				}
				{
					const TAttribute<FText> Label = LOCTEXT("Redirector_UpdateReferencers", "Update Redirector References");
					const TAttribute<FText> ToolTip = LOCTEXT("Redirector_FixUpTooltip", "Finds references to selected redirectors and resaves the referencing assets if possible, so that they reference the target of the redirector directly instead.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.ObjectRedirector");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFixUp);

					InSection.AddMenuEntry("Redirector_UpdateReferencers", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

EAssetCommandResult UAssetDefinition_Redirector::ActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	if ( ActivateArgs.ActivationMethod == EAssetActivationMethod::DoubleClicked || ActivateArgs.ActivationMethod == EAssetActivationMethod::Opened )
	{
		// Sync to the target instead of opening an editor when double clicked
		TArray<UObjectRedirector*> Redirectors = ActivateArgs.LoadObjects<UObjectRedirector>();
		if ( Redirectors.Num() > 0 )
		{
			RedirectorUtilities::FindTargets(Redirectors);
			return EAssetCommandResult::Handled;
		}
	}

	return EAssetCommandResult::Unhandled;
}

#undef LOCTEXT_NAMESPACE
