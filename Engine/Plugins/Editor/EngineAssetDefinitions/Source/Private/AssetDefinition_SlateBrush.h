// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Slate/SlateBrushAsset.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_SlateBrush.generated.h"

UCLASS()
class UAssetDefinition_SlateBrush : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SlateBrush", "Slate Brush"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(105, 165, 60)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USlateBrushAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::UI, NSLOCTEXT("AssetDefinition", "SlateBrush_SubMenu", "Slate"), ECategoryMenuType::Section) };
		return Categories;
	}
};
