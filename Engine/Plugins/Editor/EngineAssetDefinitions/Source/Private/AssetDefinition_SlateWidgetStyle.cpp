// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SlateWidgetStyle.h"
#include "Toolkits/SimpleAssetEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_SlateWidgetStyle)

#define LOCTEXT_NAMESPACE "UAssetDefinition_SlateWidgetStyle"

EAssetCommandResult UAssetDefinition_SlateWidgetStyle::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	struct Local
	{
		static TArray<UObject*> GetSubObjects(const TArray<UObject*>& InObjects)
		{
			TArray<UObject*> SubObjects;
			for(UObject* Object : InObjects)
			{
				auto Style = Cast<USlateWidgetStyleAsset>(Object);
				if(Style && Style->CustomStyle)
				{
					SubObjects.Add(Style->CustomStyle);
				}
			}
			return SubObjects;
		}
	};

	const TArray<UObject*> Objects(OpenArgs.LoadObjects<USlateWidgetStyleAsset>());
	FSimpleAssetEditor::CreateEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Objects, FSimpleAssetEditor::FGetDetailsViewObjects::CreateStatic(&Local::GetSubObjects));

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
