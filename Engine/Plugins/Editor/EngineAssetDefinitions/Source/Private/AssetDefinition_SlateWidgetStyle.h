// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateWidgetStyleAsset.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_SlateWidgetStyle.generated.h"

UCLASS()
class UAssetDefinition_SlateWidgetStyle : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SlateStyle", "Slate Widget Style"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(62, 140, 35)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USlateWidgetStyleAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::UI, NSLOCTEXT("AssetDefinition", "SlateStyle_SubMenu", "Slate"), ECategoryMenuType::Section) };
		return Categories;
	}
	
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
