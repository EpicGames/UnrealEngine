// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SparseVolumeTexture.h"

#include "Factories/SparseVolumeTextureMaterialFactory.h"
#include "ContentBrowserMenuContexts.h"
#include "IAssetTools.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_SparseVolumeTexture)

#define LOCTEXT_NAMESPACE "UAssetDefinition_SparseVolumeTexture"

namespace MenuExtension_SparseVolumeTexture
{
	static void ExecuteCreateMaterial(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	
		IAssetTools::Get().CreateAssetsFrom<USparseVolumeTexture>(
			CBContext->LoadSelectedObjects<USparseVolumeTexture>(),
			GetDefault<USparseVolumeTextureMaterialFactoryNew>()->GetSupportedClass(),
			TEXT("_Mat"),
			[](USparseVolumeTexture* SourceObject)
			{
				USparseVolumeTextureMaterialFactoryNew* Factory = NewObject<USparseVolumeTextureMaterialFactoryNew>();
				Factory->InitialTexture = SourceObject;
				return Factory;
			}
		);
	}

	static void ExecuteCreateMaterialInstance(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
	
		IAssetTools::Get().CreateAssetsFrom<USparseVolumeTexture>(
			CBContext->LoadSelectedObjects<USparseVolumeTexture>(),
			GetDefault<USparseVolumeTextureMaterialInstanceFactoryNew>()->GetSupportedClass(),
			TEXT("_MIC"),
			[](USparseVolumeTexture* SourceObject)
			{
				USparseVolumeTextureMaterialInstanceFactoryNew* Factory = NewObject<USparseVolumeTextureMaterialInstanceFactoryNew>();
				Factory->InitialTexture = SourceObject;
				return Factory;
			}
		);
	}

	static void ExtendAssetActions()
	{
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
		{
			UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USparseVolumeTexture::StaticClass())->AddDynamicSection(
				NAME_None,
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* InMenu)
					{
						const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
						if (Context && Context->SelectedAssets.Num() > 0)
						{
							FToolMenuSection& InSection = InMenu->FindOrAddSection("GetAssetActions");
							{
								const TAttribute<FText> Label = LOCTEXT("Texture_CreateMaterial", "Create Material");
								const TAttribute<FText> ToolTip = LOCTEXT("Texture_CreateMaterialTooltip", "Creates a new material using this sparse volume texture.");
								const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Material");
								const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateMaterial);
								InSection.AddMenuEntry("Texture_CreateMaterial", Label, ToolTip, Icon, UIAction);
							}
						}
					}
				)
			);

			UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USparseVolumeTexture::StaticClass())->AddDynamicSection(
				NAME_None,
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* InMenu)
					{
						const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
						if (Context && Context->SelectedAssets.Num() > 0)
						{
							FToolMenuSection& InSection = InMenu->FindOrAddSection("GetAssetActions");
							{
								const TAttribute<FText> Label = LOCTEXT("Texture_CreateMaterialInstance", "Create Material Instance");
								const TAttribute<FText> ToolTip = LOCTEXT("Texture_CreateMaterialInstanceTooltip", "Creates a new material instance using this sparse volume texture.");
								const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Material");
								const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateMaterialInstance);
								InSection.AddMenuEntry("Texture_CreateMaterialInstance", Label, ToolTip, Icon, UIAction);
							}
						}
					}
				)
			);
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(
		EDelayedRegisterRunPhase::EndOfEngineInit,
		[]
		{
			UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateStatic(&ExtendAssetActions));
		}
	);
}

#undef LOCTEXT_NAMESPACE
