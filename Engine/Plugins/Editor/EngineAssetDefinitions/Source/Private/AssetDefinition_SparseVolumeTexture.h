// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "AssetDefinitionDefault.h"
#include "Factories/Factory.h"

#include "AssetDefinition_SparseVolumeTexture.generated.h"

UCLASS()
class UAssetDefinition_SparseVolumeTexture : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SparseVolumeTexture", "BaseSparseVolumeTexture"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 128, 64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USparseVolumeTexture::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Texture };
		return Categories;
	}
	virtual bool CanImport() const override { return true; }
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_StaticSparseVolumeTexture : public UAssetDefinition_SparseVolumeTexture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_StaticSparseVolumeTexture", "StaticSparseVolumeTexture"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UStaticSparseVolumeTexture::StaticClass(); }
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_AnimatedSparseVolumeTexture : public UAssetDefinition_SparseVolumeTexture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimatedSparseVolumeTexture", "AnimatedSparseVolumeTexture"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimatedSparseVolumeTexture::StaticClass(); }
	// UAssetDefinition End
};
