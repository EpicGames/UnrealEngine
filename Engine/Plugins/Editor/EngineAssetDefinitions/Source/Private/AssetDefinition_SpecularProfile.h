// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/SpecularProfile.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_SpecularProfile.generated.h"

UCLASS()
class UAssetDefinition_SpecularProfile : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SpecularProfile", "Specular Profile"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128, 128, 255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USpecularProfile::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
    {
    	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material, NSLOCTEXT("Material", "MaterialAssetSubMenu_Profiles", "Profiles"), ECategoryMenuType::Section) };
    	return Categories;
    }
	// UAssetDefinition End
};
