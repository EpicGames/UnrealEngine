// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_StaticMesh.h"

#include "Algo/Accumulate.h"
#include "EditorSupportDelegates.h"
#include "Engine/StaticMeshSourceData.h"
#include "StaticMeshEditorModule.h"
#include "FbxMeshUtils.h"
#include "Misc/MessageDialog.h"
#include "StaticMeshCompiler.h"
#include "StaticMeshResources.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_StaticMesh)

#define LOCTEXT_NAMESPACE "UAssetDefinition_StaticMesh"

UThumbnailInfo* UAssetDefinition_StaticMesh::LoadThumbnailInfo(const FAssetData& InAssetData) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAssetData.GetAsset(), USceneThumbnailInfo::StaticClass());
}

FAssetOpenSupport UAssetDefinition_StaticMesh::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit || OpenSupportArgs.OpenMethod == EAssetOpenMethod::View); 
}

EAssetCommandResult UAssetDefinition_StaticMesh::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	IStaticMeshEditorModule* StaticMeshEditorModule = &FModuleManager::LoadModuleChecked<IStaticMeshEditorModule>( "StaticMeshEditor" );
	
	for (UStaticMesh* Mesh : OpenArgs.LoadObjects<UStaticMesh>())
	{
		StaticMeshEditorModule->CreateStaticMeshEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Mesh);
	}

	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_StaticMesh
{
	// When copying and pasting LOD data between meshes, we don't actually put the data onto the clipboard it's just
	// data stored here and referenced when executing paste.
    static TWeakObjectPtr<UStaticMesh> LODCopyMesh;

	static TAutoConsoleVariable<int32> CVarEnableSaveGeneratedLODsInPackage(
		TEXT("r.StaticMesh.EnableSaveGeneratedLODsInPackage"),
		0,
		TEXT("Enables saving generated LODs in the Package.\n") \
		TEXT("0 - Do not save (and hide this menu option) [default].\n") \
		TEXT("1 - Enable this option and save the LODs in the Package.\n"),
		ECVF_Default
	);
		
	static void ExecuteSaveGeneratedLODsInPackage(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			for (UStaticMesh* StaticMesh : CBContext->LoadSelectedObjects<UStaticMesh>())
			{
				StaticMesh->GenerateLodsInPackage();
			}
		}
	}

	static void ExecuteRemoveVertexColors(const FToolMenuContext& InContext)
	{
		const FText WarningMessage = LOCTEXT("Warning_RemoveVertexColors", "Are you sure you want to remove vertex colors from all selected meshes?  There is no undo available.");
		if (FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage) == EAppReturnType::Yes)
		{
			FScopedSlowTask SlowTask(1.0f, LOCTEXT("RemovingVertexColors", "Removing Vertex Colors"));
			if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
            {
            	for (UStaticMesh* StaticMesh : CBContext->LoadSelectedObjects<UStaticMesh>())
            	{
					StaticMesh->RemoveVertexColors();
				}
			}
		}
	}
	
	// Nanite Section
	//=================================================================
	
	const static FName NAME_NaniteEnabled("NaniteEnabled");
	static bool IsNaniteEnabled(const FAssetData& StaticMeshAsset)
	{
		bool bNaniteEnabled = false;
		return StaticMeshAsset.GetTagValue<bool>(NAME_NaniteEnabled, bNaniteEnabled) && bNaniteEnabled;
	}
	
	void ModifyNaniteEnable(const TArray<UStaticMesh*>& Objects, bool bNaniteEnable)
	{
		TArray<UStaticMesh*> Meshes;
		Meshes.Reserve(Objects.Num());

		for (UStaticMesh* Mesh: Objects)
		{
			if (Mesh && Mesh->GetNaniteSettings().bEnabled != bNaniteEnable)
			{
				Meshes.Add(Mesh);
			}
		}

		FStaticMeshCompilingManager::FFinishCompilationOptions FinishCompilationOptions;
		FinishCompilationOptions.bIncludeDependentMeshes = true;
		FStaticMeshCompilingManager::Get().FinishCompilation(Meshes, FinishCompilationOptions);

		for (UStaticMesh* Mesh : Meshes)
		{
			Mesh->GetNaniteSettings().bEnabled = bNaniteEnable;
		}

		UStaticMesh::BatchBuild(Meshes);

		for (UStaticMesh* Mesh : Meshes)
		{
			Mesh->MarkPackageDirty();
			Mesh->OnMeshChanged.Broadcast();
		}
	}

	void ExecuteNaniteEnable(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UStaticMesh*> RegularMeshes = Context->LoadSelectedObjectsIf<UStaticMesh>([](const FAssetData& AssetData) {
			return !IsNaniteEnabled(AssetData);
		});
		ModifyNaniteEnable(RegularMeshes, true);
	}

	void ExecuteNaniteDisable(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
    	TArray<UStaticMesh*> NaniteMeshes = Context->LoadSelectedObjectsIf<UStaticMesh>([](const FAssetData& AssetData) {
    		return IsNaniteEnabled(AssetData);
    	});
		ModifyNaniteEnable(NaniteMeshes, false);
	}

	static void GetNaniteMenu(UToolMenu* Menu)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*Menu);
	
		FToolMenuSection& Section = Menu->FindOrAddSection("NaniteActions");

		{
			const TAttribute<FText> Label = LOCTEXT("StaticMesh_NaniteToggle", "Nanite");
			const TAttribute<FText> ToolTip = LOCTEXT("StaticMesh_NaniteToggleTooltip", "Toggle Nanite support on the selected meshes.");
			const FSlateIcon Icon = FSlateIcon();

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
				{
					const bool bContainsTrue = Context->SelectedAssets.ContainsByPredicate([](const FAssetData& InAsset) { return IsNaniteEnabled(InAsset); });
					const bool bContainsFalse = Context->SelectedAssets.ContainsByPredicate([](const FAssetData& InAsset) { return !IsNaniteEnabled(InAsset); });

					if ((bContainsTrue && bContainsFalse) || bContainsTrue)
					{
						ExecuteNaniteDisable(InContext);
					}
					else
					{
						ExecuteNaniteEnable(InContext);
					}
				}
			});
			UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([](const FToolMenuContext& InContext)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
				{
					const bool bContainsTrue = Context->SelectedAssets.ContainsByPredicate([](const FAssetData& InAsset) { return IsNaniteEnabled(InAsset); });
					const bool bContainsFalse = Context->SelectedAssets.ContainsByPredicate([](const FAssetData& InAsset) { return !IsNaniteEnabled(InAsset); });
					return bContainsTrue && bContainsFalse ? ECheckBoxState::Undetermined : (bContainsTrue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
				}

				return ECheckBoxState::Undetermined;
			});
			
			Section.AddMenuEntry("StaticMesh_NaniteToggle", Label, ToolTip, Icon, UIAction, EUserInterfaceActionType::ToggleButton);
		}

		Section.AddSeparator("Nanite_EnableDisableOptions");

		const int32 NaniteMeshes = Algo::Accumulate(Context->SelectedAssets, 0, [](int32 Value, const FAssetData& StaticMeshAsset)
			{
				bool bNaniteEnabled = false;
                return (StaticMeshAsset.GetTagValue<bool>(NAME_NaniteEnabled, bNaniteEnabled) && bNaniteEnabled) ? Value + 1 : Value;
			});

		{
			const int32 RegularMeshes = Context->SelectedAssets.Num() - NaniteMeshes;
			
			const TAttribute<FText> Label = FText::Format(LOCTEXT("StaticMesh_NaniteEnableAll", "Enable Nanite ({0} Meshes)"), RegularMeshes);
			const TAttribute<FText> ToolTip = LOCTEXT("StaticMesh_NaniteEnableAllTooltip", "Enables support for Nanite on the selected meshes.");
			const FSlateIcon Icon = FSlateIcon();
			
			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNaniteEnable);
			UIAction.CanExecuteAction.BindLambda([RegularMeshes](const FToolMenuContext&) { return RegularMeshes > 0; });
			Section.AddMenuEntry("StaticMesh_EnableNanite", Label, ToolTip, Icon, UIAction);
		}
		
		{
			const TAttribute<FText> Label = FText::Format(LOCTEXT("StaticMesh_NaniteDisableAll", "Disable Nanite ({0} Meshes)"), NaniteMeshes);
			const TAttribute<FText> ToolTip = LOCTEXT("StaticMesh_NaniteDisableAllTooltip", "Disables support for Nanite on the selected meshes.");
			const FSlateIcon Icon = FSlateIcon();
			
			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNaniteDisable);
			UIAction.CanExecuteAction.BindLambda([NaniteMeshes](const FToolMenuContext&) { return NaniteMeshes > 0; });
			Section.AddMenuEntry("StaticMesh_DisableNanite", Label, ToolTip, Icon, UIAction);
		}
	}

	// LOD Section
	//=================================================================
	
	static void ExecuteImportMeshLOD(const FToolMenuContext&, FAssetData StaticMeshAsset, int32 LOD, bool bReimportWithNewFile)
	{
		if (UStaticMesh* Mesh = Cast<UStaticMesh>(StaticMeshAsset.GetAsset()))
		{
			FbxMeshUtils::ImportMeshLODDialog(Mesh, LOD, true, bReimportWithNewFile);
		}
	}

	static void ExecuteImportHiResMesh(const FToolMenuContext& InContext)
	{
		if (UStaticMesh* StaticMesh = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<UStaticMesh>(InContext))
		{
			if (FbxMeshUtils::ImportStaticMeshHiResSourceModelDialog(StaticMesh))
			{
				FEditorDelegates::RefreshEditor.Broadcast();
				FEditorSupportDelegates::RedrawAllViewports.Broadcast();
			}
		}
	}

	static void ExecuteReimportHiResMeshWithNewFile(const FToolMenuContext& InContext)
	{
		if (UStaticMesh* StaticMesh = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<UStaticMesh>(InContext))
		{
			StaticMesh->GetHiResSourceModel().SourceImportFilename = FString();
			if (FbxMeshUtils::ImportStaticMeshHiResSourceModelDialog(StaticMesh))
			{
				FEditorDelegates::RefreshEditor.Broadcast();
				FEditorSupportDelegates::RedrawAllViewports.Broadcast();
			}
		}
	}

	static void ExecuteRemoveHiResMesh(const FToolMenuContext& InContext)
	{
		if (UStaticMesh* StaticMesh = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<UStaticMesh>(InContext))
		{
			if (FbxMeshUtils::RemoveStaticMeshHiRes(StaticMesh))
			{
				FEditorDelegates::RefreshEditor.Broadcast();
				FEditorSupportDelegates::RedrawAllViewports.Broadcast();
			}
		}
	}

	const static FName NAME_HasHiResMesh("HasHiResMesh");
	static TOptional<bool> HasHiResMesh(const FAssetData& StaticMeshAsset)
	{
		bool bHasHiResMesh = false;
		if (StaticMeshAsset.GetTagValue<bool>(NAME_HasHiResMesh, bHasHiResMesh))
		{
			return bHasHiResMesh;
		}

		return TOptional<bool>();
	}

	static void GetImportHiResMenu(UToolMenu* Menu)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*Menu);
		check(Context->SelectedAssets.Num() == 1);
		const FAssetData& StaticMeshAsset = Context->SelectedAssets[0];

		FToolMenuSection& Section = Menu->FindOrAddSection("HighResMeshActions");

		const TOptional<bool> bHasHighResMesh = HasHiResMesh(StaticMeshAsset);
		
		// Import / Reimport
		{
			if (!bHasHighResMesh.Get(false))
			{
				const TAttribute<FText> Label = LOCTEXT("Import_HighRes", "Import High Res...");
				const TAttribute<FText> ToolTip = LOCTEXT("Import_HighResTip", "Adds a high resolution version of the mesh. Used instead of LOD0 by systems that can benefit from more detailed geometry (Nanite, modeling, etc.).");
				const FSlateIcon Icon = FSlateIcon();
		
				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteImportHiResMesh);
				Section.AddMenuEntry("Import_HighRes", Label, ToolTip, Icon, UIAction);
			}
			else
			{
				{
					const TAttribute<FText> Label = LOCTEXT("Reimport_HighRes", "Reimport High Res");
					const TAttribute<FText> ToolTip = LOCTEXT("Reimport_HighRes_Tip", "Reimport over existing the existing high res mesh.");
					const FSlateIcon Icon = FSlateIcon();
		
					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteImportHiResMesh);
					Section.AddMenuEntry("Reimport_HighRes", Label, ToolTip, Icon, UIAction);
				}
				{
					const TAttribute<FText> Label = LOCTEXT("Reimport_HighRes_WithNewFile", "Reimport High Res with new file...");
					const TAttribute<FText> ToolTip = LOCTEXT("Reimport_HighRes_WithNewFileTip", "Reimport the high resolution version of the mesh with a new file.");
					const FSlateIcon Icon = FSlateIcon();
		
					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteReimportHiResMeshWithNewFile);
					Section.AddMenuEntry("Reimport_HighRes_WithNewFile", Label, ToolTip, Icon, UIAction);
				}
			}
		}

		// Remove
		{
			const TAttribute<FText> Label = LOCTEXT("Remove_HighRes", "Remove High Res");
			const TAttribute<FText> ToolTip = LOCTEXT("Remove_HighRes_Tip", "Remove the high res version of the mesh");
			const FSlateIcon Icon = FSlateIcon();
		
			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteRemoveHiResMesh);
			UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([bHasHighResMesh](const FToolMenuContext& Context) { return bHasHighResMesh.Get(true); });
			Section.AddMenuEntry("Remove_HighRes", Label, ToolTip, Icon, UIAction);
		}
	}

	static void ExecuteCopyLODSettings(const FToolMenuContext& InContext)
	{
		if (UStaticMesh* StaticMesh = UContentBrowserAssetContextMenuContext::LoadSingleSelectedAsset<UStaticMesh>(InContext))
		{
			LODCopyMesh = StaticMesh;
		}
	}

	static void ExecutePasteLODSettings(const FToolMenuContext& InContext)
	{
		if (!LODCopyMesh.IsValid())
		{
			return;
		}

		// Retrieve LOD settings from source mesh
		struct FLODSettings
		{
			FMeshReductionSettings		ReductionSettings;
			float						ScreenSize;
		};

		TArray<FLODSettings> LODSettings;
		LODSettings.AddZeroed(FMath::Min(LODCopyMesh->GetNumSourceModels(), MAX_STATIC_MESH_LODS));
		for (int32 i = 0; i < LODSettings.Num(); i++)
		{
			LODSettings[i].ReductionSettings = LODCopyMesh->GetSourceModel(i).ReductionSettings;
			LODSettings[i].ScreenSize = LODCopyMesh->GetSourceModel(i).ScreenSize.Default;
		}

		const bool bAutoComputeLODScreenSize = LODCopyMesh->GetAutoComputeLODScreenSize();
		
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		// Copy LOD settings over to selected objects in content browser (meshes)
		for (UStaticMesh* Mesh : Context->LoadSelectedObjects<UStaticMesh>())
		{
			const int32 LODCount = LODSettings.Num();
			Mesh->SetNumSourceModels(LODCount);

			for (int32 i = 0; i < LODCount; i++)
			{
				FStaticMeshSourceModel& SrcModel = Mesh->GetSourceModel(i);
				SrcModel.ReductionSettings = LODSettings[i].ReductionSettings;
				SrcModel.ScreenSize.Default = LODSettings[i].ScreenSize;
			}

			Mesh->SetAutoComputeLODScreenSize(bAutoComputeLODScreenSize);

			Mesh->PostEditChange();
			Mesh->MarkPackageDirty();
		}
	}

	static bool CanPasteLODSettings(const FToolMenuContext& InContext)
	{
		return LODCopyMesh.IsValid();
	}
	
	const static FName NAME_LODs("LODs");
    static int32 GetNumberOfLODs(const FAssetData& StaticMeshAsset)
    {
    	int32 NumberOfLODs = 0;
    	return StaticMeshAsset.GetTagValue<int32>(NAME_LODs, NumberOfLODs) ? NumberOfLODs : 0;
    }
	
	static void GetImportLODMenu(UToolMenu* Menu, bool bReimportWithNewFile)
	{
	    const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*Menu);
    	check(Context->SelectedAssets.Num() == 1);
    	
		FToolMenuSection& Section = Menu->FindOrAddSection("ImportLODMenu");
		
		const FAssetData& StaticMeshAsset = Context->SelectedAssets[0];

		//Add 1 so we can Add LOD when we are not in "Reimport With New File" submenu
    	const int32 OriginalLodCount = GetNumberOfLODs(StaticMeshAsset);
    	const int32 LodCount = OriginalLodCount+ (bReimportWithNewFile ? 0 : 1);
    	
		for (int32 LOD = 1; LOD < LodCount; ++LOD)
		{
			FText Label = FText::Format(LOCTEXT("Reimport LOD (number)", "Reimport LOD {0}"), LOD);
			FText ToolTip = LOCTEXT("ReimportTip", "Reimport over existing LOD");
			if (LOD == OriginalLodCount)
			{
				Label = FText::Format(LOCTEXT("Add LOD (number)", "Add LOD {0}"), LOD);
				ToolTip = LOCTEXT("NewImportTip", "Import new LOD");
			}

			{
				const FSlateIcon Icon = FSlateIcon();
				const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteImportMeshLOD, StaticMeshAsset, LOD, bReimportWithNewFile);
				Section.AddMenuEntry(*FString::Printf(TEXT("LOD_%d"), LOD), Label, ToolTip, Icon, UIAction);
			}
		}
	}

	static void GetLODMenu(UToolMenu* Menu)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(*Menu);
    	FToolMenuSection& Section = Menu->FindOrAddSection("LODActions");

    	if (Context->SelectedAssets.Num() == 1)
    	{
    		constexpr bool bReimportWithNewFile = true;
    		Section.AddSubMenu(
    			"ImportLODsMenu",
				NSLOCTEXT("AssetTypeActions_StaticMesh", "StaticMesh_ImportLOD", "Import LOD"),
				NSLOCTEXT("AssetTypeActions_StaticMesh", "StaticMesh_ImportLODtooltip", "Imports meshes into the LODs"),
				FNewToolMenuDelegate::CreateStatic(&GetImportLODMenu, !bReimportWithNewFile)
			);

    		Section.AddSubMenu(
    			"ReimportLODsWithNewFileMenu",
				NSLOCTEXT("AssetTypeActions_StaticMesh", "StaticMesh_ImportLODWithNewFile", "Reimport LOD With New File"),
				NSLOCTEXT("AssetTypeActions_StaticMesh", "StaticMesh_ImportLODWithNewFiletooltip", "Reimports meshes LODs with new file."),
				FNewToolMenuDelegate::CreateStatic(&GetImportLODMenu, bReimportWithNewFile)
			);
    	}

		Section.AddSeparator("CopyPaste");
	    
		{
			const TAttribute<FText> Label = LOCTEXT("StaticMesh_CopyLOD", "Copy LOD");
			const TAttribute<FText> ToolTip = LOCTEXT("StaticMesh_CopyLODTooltip", "Copies the LOD settings from the selected mesh.");
			const FSlateIcon Icon = FSlateIcon();

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCopyLODSettings);
			UIAction.CanExecuteAction.BindLambda([](const FToolMenuContext& Context) { return UContentBrowserAssetContextMenuContext::GetNumAssetsSelected(Context) == 1; });
			Section.AddMenuEntry("StaticMesh_CopyLOD", Label, ToolTip, Icon, UIAction);
		}

	    {
			FText Label = FText(LOCTEXT("StaticMesh_PasteLOD", "Paste LOD"));
			if (LODCopyMesh.IsValid())
			{
				Label = FText::Format(LOCTEXT("StaticMesh_PasteLODWithName", "Paste LOD from {0}"), FText::FromString(LODCopyMesh->GetName()));
			}
			
			const TAttribute<FText> ToolTip = LOCTEXT("StaticMesh_PasteLODToltip", "Pastes LOD settings to the selected mesh(es).");
			const FSlateIcon Icon = FSlateIcon();

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecutePasteLODSettings);
			UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanPasteLODSettings);
			Section.AddMenuEntry("StaticMesh_PasteLOD", Label, ToolTip, Icon, UIAction);
	    }
 
    	if (Context->SelectedAssets.Num() == 1)
    	{
    		Section.AddSeparator("HighRes");
			Section.AddSubMenu(
				"ImportHighResMenu",
				NSLOCTEXT("AssetTypeActions_StaticMesh", "StaticMesh_HiRes", "High Res"),
				NSLOCTEXT("AssetTypeActions_StaticMesh", "StaticMesh_HiRestooltip", "High resolution version of the mesh. Used instead of LOD0 by systems that can benefit from more detailed geometry (Nanite, modeling, etc.)"),
				FNewToolMenuDelegate::CreateStatic(&GetImportHiResMenu)
			);
    	}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UStaticMesh::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection);
				if (Context && Context->bCanBeModified)
				{
					if (CVarEnableSaveGeneratedLODsInPackage.GetValueOnGameThread() != 0)
					{
						const TAttribute<FText> Label = LOCTEXT("ObjectContext_SaveGeneratedLODsInPackage", "Save Generated LODs");
						const TAttribute<FText> ToolTip = LOCTEXT("ObjectContext_SaveGeneratedLODsInPackageTooltip", "Run the mesh reduce and save the generated LODs as part of the package.");
						const FSlateIcon Icon = FSlateIcon();

						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteSaveGeneratedLODsInPackage);
						InSection.AddMenuEntry("ObjectContext_SaveGeneratedLODsInPackage", Label, ToolTip, Icon, UIAction);
					}
					{
						const TAttribute<FText> Label = LOCTEXT("ObjectContext_ClearVertexColors", "Remove Vertex Colors");
						const TAttribute<FText> ToolTip = LOCTEXT("ObjectContext_ClearVertexColorsTooltip", "Removes vertex colors from all LODS in all selected meshes.");
						const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.RemoveVertexColors");

						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteRemoveVertexColors);
						InSection.AddMenuEntry("ObjectContext_ClearVertexColors", Label, ToolTip, Icon, UIAction);
					}
					{
						InSection.AddSubMenu(
							"StaticMesh_NaniteMenu",
							LOCTEXT("StaticMesh_NaniteMenu", "Nanite"),
							LOCTEXT("StaticMesh_NaniteTooltip", "Nanite Options and Tools"),
							FNewToolMenuDelegate::CreateStatic(&GetNaniteMenu),
							false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Adjust")
						);
					}
					{
						InSection.AddSubMenu(
							"StaticMesh_LODMenu",
							LOCTEXT("StaticMesh_LODMenu", "Level Of Detail"),
							LOCTEXT("StaticMesh_LODTooltip", "LOD Options and Tools"),
							FNewToolMenuDelegate::CreateStatic(&GetLODMenu),
							false,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.LOD")
						);
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
