// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/StaticMesh.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_StaticMesh.generated.h"

UCLASS()
class UAssetDefinition_StaticMesh : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_StaticMesh", "Static Mesh"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 255, 255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UStaticMesh::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Basic };
		return Categories;
	}
	virtual bool CanImport() const override { return true; }
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
