// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Particles/SubUVAnimation.h"

#include "AssetDefinition_SubUVAnimation.generated.h"

UCLASS()
class UAssetDefinition_SubUVAnimation : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SubUVAnimation", "Sub UV Animation"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255,255,255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USubUVAnimation::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Misc, NSLOCTEXT("AssetDefinition", "SubUVAnimation_SubMenu", "Other"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	// UAssetDefinition End
};
