// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SubsurfaceProfile.h"

#include "Engine/SubsurfaceProfile.h"
#include "Engine/SubsurfaceScatteringMath.h"

#include "TextureResource.h"
#include "Engine/Texture2D.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"

#include "Widgets/Images/SImage.h"

#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SComboButton.h"

#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_SubsurfaceProfile)

#define LOCTEXT_NAMESPACE "AssetDefinition_SubsurfaceProfile"

const FName FSubsurfaceProfileEditorAppName = FName(TEXT("SubsurfaceProfileEditorApp"));

//////////////////////////////////////////////////////////////////////////

struct FSubsurfaceProfileEditorTabs
{
	// Tab identifiers
	static const FName DetailsID;
};

struct FSubsurfaceProfileTexturePreviewContext
{
	/** CPU copy of the image */
	TArray<FColor> Pixels;

	/** Texture and brush */
	UTexture2D* Texture = nullptr;
	FSlateBrush Brush;

	/** SImage Reference */
	TSharedPtr<SImage> Image;

	void UpdateTexture(int32 Width, int32 Height)
	{
		if (!Texture || 
			Texture->GetSizeX()!= Width || 
			Texture->GetSizeY()!= Height ||
			Pixels.Num() != Width * Height)
		{
			if (Texture)
			{
				Texture->RemoveFromRoot();
			}

			Texture = UTexture2D::CreateTransient(Width, Height, PF_B8G8R8A8);
			Texture->AddToRoot();
			Texture->SRGB = true;
			Texture->Filter = TF_Bilinear;
			Texture->AddressX = TA_Clamp;
			Texture->AddressY = TA_Clamp;
		}

		if (Texture && Pixels.Num() > 0)
		{
			checkf(Width * Height == Pixels.Num(), TEXT("Texture size does not match pixel color dimension"));
			
			FTexturePlatformData* PlatformData = Texture->GetPlatformData();
			if (PlatformData && PlatformData->Mips.Num() > 0)
			{
				FByteBulkData& BulkData = PlatformData->Mips[0].BulkData;
				void* Data = BulkData.Lock(LOCK_READ_WRITE);
				FMemory::Memcpy(Data, Pixels.GetData(), Width * Height * sizeof(FColor));
				BulkData.Unlock();
				Texture->UpdateResource();
			}
		}

		Brush.SetResourceObject(Texture);
		Brush.ImageSize = FVector2D(Width, Height);
		Brush.DrawAs = ESlateBrushDrawType::Image;
		Brush.Tiling = ESlateBrushTileType::NoTile;

		Image->SetImage(&Brush);
	}
};

struct FSubsurfaceProfilePreset
{
	FText Name;
	FLinearColor SurfaceAlbedo;
	FLinearColor MeanFreePathColor;
	float MeanFreePathDistance;
};
static const TArray<FSubsurfaceProfilePreset> GetSubsurfaceProfilePresets();

struct FPreviewUtil
{
	static constexpr float MaxTransmissionDistance = 5.0f; // SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE
	static constexpr float ExtinctionScaleConstant = 1.0f;
	static constexpr float CmToMm = 10.0f;

	static FLinearColor ComputeDiffuseMeanFreePathInMm(const FSubsurfaceProfileStruct& Settings)
	{
		ensure(Settings.bEnableMeanFreePath);

		FLinearColor SurfaceAlbedo = Settings.SurfaceAlbedo;
		FLinearColor MFP = Settings.MeanFreePathColor * Settings.MeanFreePathDistance;
		FLinearColor DMFP = SubsurfaceScattering::BurleyNormalized::GetDiffuseMeanFreePathFromMeanFreePath(SurfaceAlbedo, MFP);
		return DMFP * CmToMm;
	}

	static float CalculateThickness(float ThicknessFraction, float ExtinctionScale)
	{
		// Max transmission distance
		// D * ExtinctionScaleConstant * Coeff = MaxTransmissionDistance, as EncodedDistance = saturate(D * ExtinctionScaleConstant * Coeff/MaxTransmissionDistance);
		// D_{max} = 5.0f / (ExtinctionScaleConstant * Coeff)
		const float ScaledMaxTransmissionDistance = MaxTransmissionDistance / (ExtinctionScaleConstant * FMath::Max(ExtinctionScale, 0.001f));

		return ThicknessFraction * ScaledMaxTransmissionDistance;
	}

	static FLinearColor ComputeTransmission(float ThicknessFraction /*[0, 1]*/, const FSubsurfaceProfileStruct& Settings)
	{
		const FLinearColor SurfaceAlbedo = Settings.SurfaceAlbedo;
		const FLinearColor DiffuseMeanFreePathInMm = ComputeDiffuseMeanFreePathInMm(Settings);
		const FVector ScalingFactor = SubsurfaceScattering::BurleyNormalized::GetSearchLightDiffuseScalingFactor(SurfaceAlbedo);
		
		const float SubsurfaceScatteringUnitInCm = 0.1f;
		const float UnitScale = (Settings.WorldUnitScale * Settings.DistanceScale) / SubsurfaceScatteringUnitInCm;
		float InvUnitScale = 1.0 / UnitScale; // Scaling the unit is equivalent to inverse scaling of the profile.

		float ThicknessMm = ThicknessFraction * MaxTransmissionDistance * CmToMm * InvUnitScale;

		float QueryThicknessMm = ThicknessMm;
		FVector Transmission = SubsurfaceScattering::BurleyNormalized::TransmissionProfile(QueryThicknessMm, SurfaceAlbedo, ScalingFactor, DiffuseMeanFreePathInMm);
		FLinearColor Color(Transmission.X, Transmission.Y, Transmission.Z, 1.0f);
		Color *= Settings.TransmissionTintColor;
		Color.A = 1.0f;
		return Color.GetClamped();
	}

	// Calculate the 1d LUT for the isotropic diffusion. Normalized.
	// d >= a      , L(d) = 2E\int_{d-a}^{d+a}f(t)arccos(\frac{d^2+t^2-a^2}{2dt}) dt
	// 0 <= d < a  , L(d) = 2E (\int_{a-d}^{d+a}f(t)arccos(\frac{d^2+t^2-a^2}{2dt}) dt + pi \int _{0}^{a-d}f(t)dt) 
	// f(x) = R(x)x
	// d: distance to circle center
	// a: circle radius.
	// Use Simpson integration for simpicity. Quadratic polynomial approximation. 
	// \int_{a}^{b}f(x)dx = (b - a)/6 (f(a)+4f(m)+f(b)), m=(a+b)/2
	// Sample at circle edge shift half pixel to avoid the extremely high value at r=0 due to the large weight of Simpson and low sample count.
	static TArray<FLinearColor> Compute1DDiffusionLUT(float PixelToMm, int32 NumPixels, float CircleRadiusInMm, const FSubsurfaceProfileStruct& Settings)
	{
		const int32 IntegrateSampleCount = 32;
		const FLinearColor SurfaceAlbedo = Settings.SurfaceAlbedo;
		const FLinearColor DiffuseMeanFreePathInMm = ComputeDiffuseMeanFreePathInMm(Settings);
		const FVector ScalingFactor = SubsurfaceScattering::BurleyNormalized::GetSearchLightDiffuseScalingFactor(SurfaceAlbedo);

		auto ArcCos = [](float t, float d, float a) { 
			if (t < 1e-12f)
			{
				return 0.5f * UE_PI;
			}
			return FMath::Acos((d * d + t * t - a * a)/(2*d*t));
		};

		auto Func = [&](float t, float d, float a){
			return SubsurfaceScattering::BurleyNormalized::DiffusionProfile(
				t, FLinearColor::White, ScalingFactor, DiffuseMeanFreePathInMm) * ArcCos(t, d, a);
		};

		auto IntegrateSimpson = [&](int32 NumSample /*Event number*/, float u, float v, float d, float a) {
			float h = (v - u) / NumSample;

			FVector Sum = Func(u, d, a) + Func(v, d, a);
			// 1 4 1
			//     1 4 1
			//         ... 1
			//             1 4 1
			// 1 4 2 4 2...2 4 1
			// 0 1 2 3 4...    
			for (int32 i = 1; i < NumSample; ++i)
			{
				float t = u + i * h;
				FVector Value = Func(t, d, a);

				if (i % 2 == 0) // even number
				{
					Sum += Value * 2.0f;
				}
				else
				{
					Sum += Value * 4.0f;
				}
			}

			return (h / 3.0f) * Sum;
		};

		TArray<FLinearColor> LUT;
		LUT.SetNumUninitialized(NumPixels);

		for (int32 i = 0; i < NumPixels; ++i)
		{
			float DistanceToCenterInMm = i * PixelToMm;
			FLinearColor Color;
			if (i == 0)
			{
				// Diffusion = 1 - transmitted at center.
				FVector Transmission = SubsurfaceScattering::BurleyNormalized::TransmissionProfile(
					CircleRadiusInMm - DistanceToCenterInMm, FLinearColor::White, ScalingFactor, DiffuseMeanFreePathInMm);
				Color = FLinearColor::White - FLinearColor(Transmission.X, Transmission.Y, Transmission.Z);
			}
			else
			{
				// suppress high intensity near Circle edge due to Simpson integration has high weight on 
				// r = 0 where R(r)*r is extremely high.
				float MinBound = 0.5f * PixelToMm; 

				float InnerRadius = CircleRadiusInMm - DistanceToCenterInMm;
				float LowerBound = FMath::Max(FMath::Abs(DistanceToCenterInMm - CircleRadiusInMm), MinBound);
				FVector Diffusion = 2.0f * IntegrateSimpson(
					IntegrateSampleCount,
					LowerBound,
					DistanceToCenterInMm + CircleRadiusInMm,
					DistanceToCenterInMm,
					CircleRadiusInMm);

				if (InnerRadius > MinBound) // Internal region has an analytic solution.
				{
					FVector Transmission = SubsurfaceScattering::BurleyNormalized::TransmissionProfile(
						InnerRadius, FLinearColor::White, ScalingFactor, DiffuseMeanFreePathInMm);
					Diffusion += FVector(1 - Transmission.X, 1 - Transmission.Y, 1 - Transmission.Z);
				}

				Color = FLinearColor(Diffusion.X, Diffusion.Y, Diffusion.Z);
			}

			LUT[i] = Color;
		}

		return LUT;

	}

	static FLinearColor Sample1DDiffusionLUT(float DistanceInMm, float PixelToMm, const TArray<FLinearColor>& CircleDiffusionLUT)
	{
		int32 Num = CircleDiffusionLUT.Num();
		float Index = DistanceInMm / PixelToMm;
		int32 Index0 = FMath::Floor(Index);
		int32 Index1 = FMath::CeilToInt(Index);

		if (Index1 >= Num)
		{
			return CircleDiffusionLUT[Num - 1];
		}

		float p = Index - Index0;

		return FMath::Lerp(CircleDiffusionLUT[Index0], CircleDiffusionLUT[Index1], p);
	}

	static int32 GetIndexOfPresetByPredicate(const FSubsurfaceProfileStruct& Settings)
	{
		const TArray<FSubsurfaceProfilePreset>& Presets = GetSubsurfaceProfilePresets();

		int32 Index = Presets.IndexOfByPredicate([&](const FSubsurfaceProfilePreset& Preset)
		{
			return Preset.SurfaceAlbedo.Equals(Settings.SurfaceAlbedo, 0.01f)
				&& Preset.MeanFreePathColor.Equals(Settings.MeanFreePathColor, 0.001f)
				&& FMath::IsNearlyEqual(Preset.MeanFreePathDistance, Settings.MeanFreePathDistance, 0.001f);
		});

		return Index; // return INDEX_NONE if not match.
	}
};

static const TArray<FSubsurfaceProfilePreset> GetSubsurfaceProfilePresets()
{
	static const TArray<FSubsurfaceProfilePreset> Presets = 
	{
		// mfp = 1 / sigma_t' = 1 / (sigma_a + sigma_s')
		{LOCTEXT("PresetApple"    ,		"Apple"), FLinearColor(0.85f, 0.84f, 0.53f), FLinearColor(0.879f, 0.842f, 1.000f), 0.0496f},
		{LOCTEXT("PresetChicken1" ,	 "Chicken1"), FLinearColor(0.31f, 0.15f, 0.10f), FLinearColor(1.000f, 0.575f, 0.289f), 0.6061f},
		{LOCTEXT("PresetChicken2" ,  "Chicken2"), FLinearColor(0.32f, 0.16f, 0.10f), FLinearColor(1.000f, 0.615f, 0.400f), 0.4808f},
		{LOCTEXT("PresetCream"    ,		"Cream"), FLinearColor(0.98f, 0.90f, 0.73f), FLinearColor(0.429f, 0.579f, 1.000f), 0.0316f},
		{LOCTEXT("PresetKetchup"  ,	  "Ketchup"), FLinearColor(0.16f, 0.01f, 0.00f), FLinearColor(1.000f, 0.232f, 0.163f), 0.4149f},
		{LOCTEXT("PresetMarble"   ,	   "Marble"), FLinearColor(0.83f, 0.79f, 0.75f), FLinearColor(1.000f, 0.835f, 0.729f), 0.0456f},
		{LOCTEXT("PresetPotato"   ,	   "Potato"), FLinearColor(0.77f, 0.62f, 0.21f), FLinearColor(0.982f, 0.945f, 1.000f), 0.1493f},
		{LOCTEXT("PresetSkimmilk" ,  "Skimmilk"), FLinearColor(0.81f, 0.81f, 0.69f), FLinearColor(1.000f, 0.574f, 0.366f), 0.1426f},
		{LOCTEXT("PresetSkin1"    ,     "Skin1"), FLinearColor(0.44f, 0.22f, 0.13f), FLinearColor(1.000f, 0.735f, 0.518f), 0.1295f},
		{LOCTEXT("PresetSkin2"    ,     "Skin2"), FLinearColor(0.63f, 0.44f, 0.34f), FLinearColor(1.000f, 0.664f, 0.570f), 0.0907f},
		{LOCTEXT("PresetWholemilk", "Wholemilk"), FLinearColor(0.91f, 0.88f, 0.76f), FLinearColor(1.000f, 0.794f, 0.674f), 0.0392f}
	};

	return Presets;
}
//////////////////////////////////////////////////////////////////////////

//const FName FFlipbookEditorTabs::DetailsID(TEXT("Details"));
//const FName FFlipbookEditorTabs::ViewportID(TEXT("Viewport"));
const FName FSubsurfaceProfileEditorTabs::DetailsID(TEXT("Properties"));

//////////////////////////////////////////////////////////////////////////


class FSubsurfaceProfileEditor : public FAssetEditorToolkit
{
public:

	void InitEditor(
		const EToolkitMode::Type Mode, 
		const TSharedPtr<IToolkitHost>& InitToolkitHost,
		USubsurfaceProfile* InSubsurfaceProfile)
	{
		SubsurfaceProfile = InSubsurfaceProfile;
		CachedSettings = InSubsurfaceProfile->Settings;
		ActivePresetIndex = FPreviewUtil::GetIndexOfPresetByPredicate(CachedSettings);
		// Create the the detail view pannel and listen to property change.

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
		DetailsView->SetObject( SubsurfaceProfile );
		DetailsView->OnFinishedChangingProperties().AddSP(this, &FSubsurfaceProfileEditor::OnFinishedChangingProperties);

		// Default layout
		const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = 
			FTabManager::NewLayout("Standalone_SubsurfaceProfileEditor_Layout_v1")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(FSubsurfaceProfileEditorTabs::DetailsID, ETabState::OpenedTab)
				)
			);


		FAssetEditorToolkit::InitAssetEditor(
			Mode,
			InitToolkitHost,
			FSubsurfaceProfileEditorAppName,
			StandaloneDefaultLayout,
			/*bCreateDefaultStandaloneMenu=*/ true, 
			/*bCreateDefaultToolbar=*/ true,
			SubsurfaceProfile
		);
	}

	~FSubsurfaceProfileEditor()
	{
		if (DiffusionProfileContext.Texture)
		{
			DiffusionProfileContext.Texture->RemoveFromRoot();
		}

		if (TransmissionGradientContext.Texture)
		{
			TransmissionGradientContext.Texture->RemoveFromRoot();
		}

		if (TransmissionGraphContext.Texture)
		{
			TransmissionGraphContext.Texture->RemoveFromRoot();
		}
	}

	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		InTabManager->RegisterTabSpawner(
			FSubsurfaceProfileEditorTabs::DetailsID,
			FOnSpawnTab::CreateSP(this, &FSubsurfaceProfileEditor::SpawnTab))
			.SetDisplayName(LOCTEXT("PropertiesTab", "Details"));
	}

	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
		InTabManager->UnregisterTabSpawner(FSubsurfaceProfileEditorTabs::DetailsID);
	}

	virtual FName GetToolkitFName() const override { return TEXT("SubsurfaceProfileEditor"); }
	virtual FText GetBaseToolkitName() const override { return LOCTEXT("Toolkit_AppLabel", "Subsurface Profile Editor"); }
	virtual FString GetWorldCentricTabPrefix() const override { return LOCTEXT("Toolkit_TabPrefix", "SubsurfaceProfileEditor").ToString(); }
	virtual FLinearColor GetWorldCentricTabColorScale(void) const override { return FLinearColor::White;}
	
	virtual FName GetEditingAssetTypeName() const override{ return USubsurfaceProfile::StaticClass()->GetFName(); }
private:

	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args)
	{
		check( Args.GetTabId() == FSubsurfaceProfileEditorTabs::DetailsID);

		TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
			.Label(NSLOCTEXT("SubsurfaceProfileEditor", "PropertiesTab", "Details"))
			.OnCanCloseTab_Lambda([]() { return false; })
			[
				// Details (60%)   | Preview (40%)
				SNew(SSplitter)
					.Orientation(Orient_Horizontal)
					+SSplitter::Slot()
					.Value(0.6f)
					[
						DetailsView.ToSharedRef()
					]
					+SSplitter::Slot()
					.Value(0.4f)
					[
						// add content inside a scroll box
						SNew(SScrollBox)
						+ SScrollBox::Slot()
						[
							// ---------------
							// Preset selector
							// Diffusion
							// Difussion image
							// Transmission
							// Transmission image
							// Transmission text
							// Slider
							SNew(SVerticalBox)

							// --- Subsurface profile preset picker
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Left)
							.Padding(8.0f, 8.0f, 8.0f, 4.0f)
							[
								SNew(SBox)
								.WidthOverride(PreviewSize)
								[
									SNew(SHorizontalBox)
									+SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									.Padding(0.0f, 0.0f, 8.0f, 0.0f)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("ProfilePresetLabel", "Profile Preset:"))
										.Font(FCoreStyle::GetDefaultFontStyle("Bold",10))
									]
									+SHorizontalBox::Slot()
									.FillWidth(1.0f)
									[
										// Combo Button
										SNew(SComboButton)
										.OnGetMenuContent_Raw(this, &FSubsurfaceProfileEditor::BuildSubsurfaceProfilePresetsMenu)
										.ButtonContent()
										[
											SNew(STextBlock)
											.Text_Raw(this, &FSubsurfaceProfileEditor::GetActiveProfilePresetText)
											.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
										]
									]

								]
							] // End preset picker

							// -- Add diffusion text
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(8.0f, 8.0f, 8.0f, 2.0f)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("DiffusionLabel", "Diffusion"))
									.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
							]

							// -- Add diffusion profile image
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Left)
							.Padding(8.0f, 0.0f)
							[
								SNew(SBox)
								.WidthOverride(PreviewSize)
								.HeightOverride(PreviewSize)
								[
									SNew(SOverlay)

									+ SOverlay::Slot()
									[
										SAssignNew(DiffusionProfileContext.Image, SImage)
									]

									// Circle light info
									+ SOverlay::Slot()
									.HAlign(HAlign_Left)
									.VAlign(VAlign_Bottom)
									.Padding(4.0f, 0.0f, 0.0f, 20.0f)
									[
										SNew(STextBlock)
											.Text_Raw(this, &FSubsurfaceProfileEditor::GetResolutionText)
											.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
									]
										
									// resolution text overlay at bottom-left
									+ SOverlay::Slot()
									.HAlign(HAlign_Left)
									.VAlign(VAlign_Bottom)
									.Padding(4.0f)
									[
										SNew(STextBlock)
										.Text_Raw(this, &FSubsurfaceProfileEditor::GetDiffusionCircleLightInfoText)
										.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
									]
									
								]
							]

							// -- Add transmission text
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(8.0f, 8.0f, 8.0f, 2.0f)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("TransmissionLabel", "Transmission"))
									.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
							]

							// -- Add transmission gradient image
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Left)
							.Padding(8.0f, 0.0f)
							[
								SNew(SBox)
								.WidthOverride(PreviewSize)
								.HeightOverride(TransmissionGradientHeight)
								[
									SAssignNew(TransmissionGradientContext.Image, SImage)
								]
							]

							// -- Add transmission graph image
							+ SVerticalBox::Slot()
								.AutoHeight()
								.HAlign(HAlign_Left)
								.Padding(8.0f, 8.0f, 8.0f, 0.0f)
								[
									SNew(SBox)
									.WidthOverride(PreviewSize)
									.HeightOverride(TransmissionGraphHeight)
									[
										SAssignNew(TransmissionGraphContext.Image, SImage)
									]
								]

							// -- Add Thickness, transmission info
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Left)
							.Padding(8.0f, 8.0f, 8.0f, 2.0f)
							[
								SNew(SBox)
								.WidthOverride(PreviewSize)
								[
									SNew(SHorizontalBox)
									+SHorizontalBox::Slot()
									.FillWidth(0.4f)
									[
										SNew(STextBlock)
										.Text_Raw(this, &FSubsurfaceProfileEditor::GetThicknessText)
										.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
									]
									+SHorizontalBox::Slot()
									.FillWidth(0.6f)
									[
										SNew(STextBlock)
										.Text_Raw(this, &FSubsurfaceProfileEditor::GetTransmissionText)
										.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
									]
								]
							]

							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(8.0f, 2.0f, 8.0f, 2.0f)
							[	
								SNew(STextBlock)
								.Text_Raw(this, &FSubsurfaceProfileEditor::GetMaxTransmissionDistanceText)
								.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
							]
							// -- Add slider
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Left)
							.Padding(8.0f, 0.0f)
							[
								SNew(SBox)
								.WidthOverride(PreviewSize)
								[
									SAssignNew(ThicknessSlider, SSlider)
									.Value(0.0f)
									.OnValueChanged_Raw(this, &FSubsurfaceProfileEditor::OnThicknessChanged)
								]
							]
						]
					]
					
			];

		UpdateGraph();

		return SpawnedTab;
	}


	TSharedRef<SWidget> BuildSubsurfaceProfilePresetsMenu()
	{
		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

		const TArray<FSubsurfaceProfilePreset>& Presets = GetSubsurfaceProfilePresets();
		for (int Index = 0; Index < Presets.Num(); ++Index)
		{
			const FSubsurfaceProfilePreset& Preset = Presets[Index];
			MenuBuilder.AddMenuEntry(
				Preset.Name,
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
                    FExecuteAction::CreateRaw(this, &FSubsurfaceProfileEditor::ApplyProfilePreset, Index),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, Index] () { return ActivePresetIndex == Index;})
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton);
		}

		return MenuBuilder.MakeWidget();
	}

	FText GetActiveProfilePresetText() const
	{
		if (ActivePresetIndex != INDEX_NONE)
		{
			const TArray<FSubsurfaceProfilePreset>& Presets = GetSubsurfaceProfilePresets();
			if (Presets.IsValidIndex(ActivePresetIndex))
			{
				return Presets[ActivePresetIndex].Name;
			}
		}

		return LOCTEXT("SelectPreset", "Custom");
	}

	void ApplyProfilePreset(int32 PresetIndex)
	{
		const TArray<FSubsurfaceProfilePreset>& Presets = GetSubsurfaceProfilePresets();
		if (!SubsurfaceProfile || !Presets.IsValidIndex(PresetIndex))
		{
			return;
		}

		const FSubsurfaceProfilePreset& Preset = Presets[PresetIndex];
		const FScopedTransaction Transaction(
			FText::Format(LOCTEXT("ApplyProfilePreset","Apply Subsurface Profile Preset: {0}"), Preset.Name));
		
		SubsurfaceProfile->Modify();
		SubsurfaceProfile->Settings.SurfaceAlbedo = Preset.SurfaceAlbedo;
		SubsurfaceProfile->Settings.MeanFreePathColor = Preset.MeanFreePathColor;
		SubsurfaceProfile->Settings.MeanFreePathDistance = Preset.MeanFreePathDistance;
		
		// Notify GPU resource update.
		SubsurfaceProfile->PostEditChange();

		// Force refresh
		if (DetailsView.IsValid())
		{
			DetailsView->ForceRefresh();
		}

		CachedSettings = SubsurfaceProfile->Settings;
		ActivePresetIndex = PresetIndex;
		UpdateGraph();
	}

	void GenerateTransmissionGradientImage()
	{
		const int32 GradientWidth = ImageWidth;
		const int32 GradientHeight = 1;
		TArray<FColor>& Pixels = TransmissionGradientContext.Pixels;
		Pixels.SetNumZeroed(ImageWidth * GradientHeight);

		for (int i = 0; i < ImageWidth; ++i)
		{
			const float T = (float) i / (GradientWidth - 1);
			FLinearColor Color = FPreviewUtil::ComputeTransmission(T,CachedSettings);
			Pixels[i] = Color.GetClamped().ToFColor(true);
		}
		Pixels[ImageWidth - 1] = FColor(0, 0, 0, 0);

		// Plot the selected thickness line
		{
			const int32 X = FMath::Clamp((int32)(SelectedThicknessFraction * (GradientWidth - 1)), 0, GradientWidth - 1);
			for (int32 Y = 0; Y < GradientHeight; ++Y)
			{
				Pixels[Y * GradientWidth + X] = FColor::White;
			}
		}

		TransmissionGradientContext.UpdateTexture(GradientWidth, GradientHeight);
	}

	void GenerateTransmissionGraphImage()
	{
		const int32 GraphWidth = ImageWidth;
		const int32 GraphHeight = TransmissionGraphHeight;

		TArray<FColor>& Pixels = TransmissionGraphContext.Pixels;
		Pixels.SetNumZeroed(GraphWidth * GraphHeight);

		// Clear the back ground.
		for (int i = 0; i < GraphWidth * GraphHeight; ++i)
		{
			Pixels[i] = FColor(20, 20, 20, 255); // Distinguish from the background.
		}

		// Draw horizontal grid lines
		for (float Value : {0.25f, 0.5f, 0.75f})
		{
			int32 CurrentY = (GraphHeight - 1) - 
				FMath::Clamp((int32) FMath::Floor(Value * (GraphHeight - 1) + 0.5f), 0, GraphHeight - 1);
			for (int32 X = 0; X < GraphWidth; ++X)
			{
				Pixels[CurrentY * GraphWidth + X] = FColor(50, 50, 50); // Gray
			}
		}

		// Simple fast aliased plot. 
		auto PlotChannel = [&](int32 ChannelIndex, FColor ChannelColor)
			{
				int32 PreviousY = -1;
				for (int i = 0; i < GraphWidth; ++i)
				{
					const float T = ((float)i + 0.5f)/ (GraphWidth - 1);
					FLinearColor Transmission = FPreviewUtil::ComputeTransmission(T, CachedSettings);
					float Value = ChannelIndex == 0 ? Transmission.R : (ChannelIndex == 1 ? Transmission.G : Transmission.B);
					Value = FMath::Clamp(Value, 0.0f, 1.0f);

					int32 CurrentY = (GraphHeight - 1) - 
						FMath::Clamp((int32) FMath::Floor(Value * (GraphHeight - 1) + 0.5f), 0, GraphHeight - 1);

					if (i > 0)
					{
						int32 MinY = FMath::Min(CurrentY, PreviousY);
						int32 MaxY = FMath::Max(CurrentY, PreviousY);
						// Draw vertical bars between previous and current y to avoid gaps.
						for (int32 j = MinY; j <= MaxY; ++j)
						{
							Pixels[j * GraphWidth + i] = ChannelColor;
						}
					}
					else
					{
						Pixels[CurrentY * GraphWidth + i] = ChannelColor;
					}
					PreviousY = CurrentY;
				}
			};

		PlotChannel(0, FColor::Red);
		PlotChannel(1, FColor::Green);
		PlotChannel(2, FColor::Blue);

		// Plot the selected thickness line
		{
			const int32 X = FMath::Clamp((int32)(SelectedThicknessFraction * (GraphWidth - 1)), 0, GraphWidth - 1);
			for (int32 Y = 0; Y < GraphHeight; ++Y)
			{
				Pixels[Y * GraphWidth + X] = FColor::White;
			}
		}

		TransmissionGraphContext.UpdateTexture(ImageWidth, TransmissionGraphHeight);

	}

	void GenerateDiffusionProfileImage()
	{
		TArray<FColor>& Pixels = DiffusionProfileContext.Pixels;
		Pixels.SetNumZeroed(ImageWidth * ImageWidth);

		const int32 ImageResolution = ImageWidth;

		const FLinearColor TransmissionAtThickness = 
			FPreviewUtil::ComputeTransmission(SelectedThicknessFraction, CachedSettings);

		// The irradiance at surface thickness = 0.0 does not accept transmission tint.
		const FLinearColor PreScatteringSurfaceExitance = (SelectedThicknessInCm == 0.0f ? CachedSettings.SurfaceAlbedo : TransmissionAtThickness) * CenterLightIrradiance;

		const FLinearColor SurfaceAlbedo = CachedSettings.SurfaceAlbedo;
		const FVector ScalingFactor = 
			SubsurfaceScattering::BurleyNormalized::GetSearchLightDiffuseScalingFactor(SurfaceAlbedo);
		const FLinearColor DMFPMm = FPreviewUtil::ComputeDiffuseMeanFreePathInMm(CachedSettings);


		CircleDiffusion1DLUT = FPreviewUtil::Compute1DDiffusionLUT(PixelToMm, (int32)(Center*FMath::Sqrt(2.0f)), CenterLightRadiusMm, CachedSettings);

		for (int32 Y = 0; Y < ImageResolution; ++Y)
		{
			for (int32 X = 0; X < ImageResolution; ++X)
			{
				const float PxMm = (X - Center + 0.5f) * PixelToMm;
				const float PyMm = (Y - Center + 0.5f) * PixelToMm;
				const float DistanceFromCenterMm = FMath::Sqrt(PxMm * PxMm + PyMm * PyMm);

				FLinearColor Color;
				Color = FPreviewUtil::Sample1DDiffusionLUT(DistanceFromCenterMm, PixelToMm, CircleDiffusion1DLUT);
				Color *= PreScatteringSurfaceExitance;

				if (DistanceFromCenterMm <= CenterLightRadiusMm)
				{
					// Apply the tint from diffusion, lerp from scattered result to base color.
					Color = FMath::Lerp(SurfaceAlbedo * CenterLightIrradiance, Color, CachedSettings.Tint);
				}
				else
				{
					// Apply the tint from diffusion, lerp from scattered result to base color.
					Color = FMath::Lerp(FLinearColor::Black, Color, CachedSettings.Tint);
				}

				Color.A = 1.0f;
				Pixels[Y * ImageResolution + X] = Color.GetClamped().ToFColor(/*bSRGB=*/true);
			}
		}

		DiffusionProfileContext.UpdateTexture(ImageWidth, ImageWidth);
	}

	void OnFinishedChangingProperties(const FPropertyChangedEvent& Event)
	{

		if (Event.Property)
		{
			const FName PropName = Event.Property->GetFName();
			if (PropName == GET_MEMBER_NAME_CHECKED(FSubsurfaceProfileStruct, SurfaceAlbedo) ||
				PropName == GET_MEMBER_NAME_CHECKED(FSubsurfaceProfileStruct, MeanFreePathColor) ||
				PropName == GET_MEMBER_NAME_CHECKED(FSubsurfaceProfileStruct, MeanFreePathDistance))
			{
				if (SubsurfaceProfile)
				{
					ActivePresetIndex = FPreviewUtil::GetIndexOfPresetByPredicate(SubsurfaceProfile->Settings);
				}
				else
				{
					ActivePresetIndex = INDEX_NONE;
				}
				
			}
		}

		// redraw
		if (SubsurfaceProfile)
		{
			CachedSettings = SubsurfaceProfile->Settings;

			UpdateGraph();
		}
	}

	void OnThicknessChanged(float NewValue)
	{
		SelectedThicknessFraction = NewValue;

		UpdateGraph();
	}

	void UpdateGraph()
	{
		ResolvedMaxTransmissionDistanceInCm = FPreviewUtil::CalculateThickness(1.0f, CachedSettings.ExtinctionScale);
		SelectedThicknessInCm = FPreviewUtil::CalculateThickness(SelectedThicknessFraction, CachedSettings.ExtinctionScale);
		SelectedTransmission = FPreviewUtil::ComputeTransmission(SelectedThicknessFraction, CachedSettings);

		GenerateTransmissionGradientImage();
		GenerateTransmissionGraphImage();
		GenerateDiffusionProfileImage();
	}

	FText GetDiffusionCircleLightInfoText() const
	{
		FNumberFormattingOptions FormattingOptions;
		FormattingOptions.MaximumFractionalDigits = 2;
		float CenterLightRadiusInCm = CenterLightRadiusMm / FPreviewUtil::CmToMm;
		
		const float SubsurfaceScatteringUnitInCm = 0.1f;
		const float UnitScale = CachedSettings.WorldUnitScale / SubsurfaceScatteringUnitInCm;

		return FText::Format(
			LOCTEXT("CenterLightInfoFmt", "Irradiance: {1} W/m^2 Light Radius : {0} cm"),
			FText::AsNumber(CenterLightRadiusInCm * UnitScale, &FormattingOptions),
			FText::AsNumber(CenterLightIrradiance, &FormattingOptions));
	}

	FText GetResolutionText() const
	{
		FNumberFormattingOptions FormattingOptions;
		FormattingOptions.MaximumFractionalDigits = 2;
		float ResolutionInCm = ImageWidth * PixelToMm / FPreviewUtil::CmToMm;
		
		const float SubsurfaceScatteringUnitInCm = 0.1f;
		const float UnitScale = CachedSettings.WorldUnitScale / SubsurfaceScatteringUnitInCm;

		return FText::Format(
			LOCTEXT("ResolutionFmt", "Resolution: {0} cm x {1} cm"),
			FText::AsNumber(ResolutionInCm * UnitScale, &FormattingOptions),
			FText::AsNumber(ResolutionInCm * UnitScale, &FormattingOptions));
	}

	FText GetThicknessText() const
	{
		FNumberFormattingOptions FormattingOptions;
		FormattingOptions.MaximumFractionalDigits = 2;
		
		return FText::Format(
			LOCTEXT("ThicknessFmt", "Thickness: {0} cm"),
			FText::AsNumber(SelectedThicknessInCm, &FormattingOptions));
	}

	FText GetTransmissionText() const
	{
		FNumberFormattingOptions FormattingOptions;
		FormattingOptions.MaximumFractionalDigits = 2;

		return FText::Format(
			LOCTEXT("TransmissionFmt", "Transmission: ({0},{1},{2})"),
			FText::AsNumber(SelectedTransmission.R, &FormattingOptions),
			FText::AsNumber(SelectedTransmission.G, &FormattingOptions),
			FText::AsNumber(SelectedTransmission.B, &FormattingOptions));
	}

	FText GetMaxTransmissionDistanceText() const
	{
		FNumberFormattingOptions FormattingOptions;
		FormattingOptions.MaximumFractionalDigits = 2;

		return FText::Format(
			LOCTEXT("MaxTransmissionDistanceFmt", "Max Transmission Distance: {0} cm"),
			FText::AsNumber(ResolvedMaxTransmissionDistanceInCm, &FormattingOptions));
	}

	/** Details view to show the existing subsurface profile*/
	TSharedPtr<IDetailsView> DetailsView;

	/** The preview size is fixed to 256px. All image widths uses this*/
	static constexpr float PreviewSize = 256.0f;
	static constexpr int32 ImageWidth = 256; 

	/** Setting of the diffusion profile image. */
	const float PixelToMm = 100.0f / ImageWidth; // Target resolution (cm): 10 x 10
	const float Center = 0.5f * ImageWidth;
	const float CenterLightRadiusMm = 10.0f; // 1.0cm radius
	const float CenterLightIrradiance = 1.0f; // W/m^2

	FSubsurfaceProfileTexturePreviewContext DiffusionProfileContext;

	// The circle diffusion 1D LUT cache
	TArray<FLinearColor> CircleDiffusion1DLUT;

	static constexpr float TransmissionGradientHeight = 24.0f;
	//TSharedPtr<SImage> TransmissionGradientImage;
	FSubsurfaceProfileTexturePreviewContext TransmissionGradientContext;

	static constexpr float TransmissionGraphHeight = 100.0f;
	//TSharedPtr<SImage> TransmissionGraphImage;
	FSubsurfaceProfileTexturePreviewContext TransmissionGraphContext;

	float SelectedThicknessFraction = 0.0f;
	TSharedPtr<SSlider> ThicknessSlider;

	float SelectedThicknessInCm = 0.0;

	FLinearColor SelectedTransmission = FLinearColor::White;

	float ResolvedMaxTransmissionDistanceInCm = 5.0f;

	/** Hold the reference of the subsurface profile to modify or check modified*/
	USubsurfaceProfile* SubsurfaceProfile = nullptr;
	
	int32 ActivePresetIndex = INDEX_NONE;

	/** Cache the setting for preview drawing */
	FSubsurfaceProfileStruct CachedSettings;

};



EAssetCommandResult UAssetDefinition_SubsurfaceProfile::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (USubsurfaceProfile* Profile : OpenArgs.LoadObjects<USubsurfaceProfile>())
	{
		TSharedRef<FSubsurfaceProfileEditor> Editor = MakeShared<FSubsurfaceProfileEditor>();
		Editor->InitEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Profile);
	}

	return EAssetCommandResult::Handled;
}


#undef LOCTEXT_NAMESPACE