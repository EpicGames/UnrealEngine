// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/SubsurfaceProfile.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_SubsurfaceProfile.generated.h"

UCLASS()
class UAssetDefinition_SubsurfaceProfile : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_SubsurfaceProfile", "Subsurface Profile"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 196, 128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return USubsurfaceProfile::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
    {
    	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material, NSLOCTEXT("Material", "MaterialAssetSubMenu_Profiles", "Profiles"), ECategoryMenuType::Section) };
    	return Categories;
    }

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
