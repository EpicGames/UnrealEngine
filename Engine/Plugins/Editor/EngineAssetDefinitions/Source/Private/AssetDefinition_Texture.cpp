// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_Texture.h"

#include "AssetRegistry/AssetIdentifier.h"
#include "ContentBrowserMenuContexts.h"
#include "Engine/Texture2D.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Factories/MaterialFactoryNew.h"
#include "Interfaces/ITextureEditorModule.h"

#include "ToolMenu.h"
#include "ToolMenus.h"
#include "IAssetTools.h"
#include "ToolMenuSection.h"
#include "VirtualTexturingEditorModule.h"
#include "Algo/AnyOf.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "TextureAssetActions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Texture)

#define LOCTEXT_NAMESPACE "UAssetDefinition_Texture"

EAssetCommandResult UAssetDefinition_Texture::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UTexture* Texture : OpenArgs.LoadObjects<UTexture>())
	{
		ITextureEditorModule* TextureEditorModule = &FModuleManager::LoadModuleChecked<ITextureEditorModule>("TextureEditor");
		TextureEditorModule->CreateTextureEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Texture);
	}
	
	return EAssetCommandResult::Handled;
}

FAssetOpenSupport UAssetDefinition_Texture::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit || OpenSupportArgs.OpenMethod == EAssetOpenMethod::View); 
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_Texture
{
	static void ExecuteCreateMaterial(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		IAssetTools::Get().CreateAssetsFrom<UTexture>(
			CBContext->LoadSelectedObjects<UTexture>(), UMaterial::StaticClass(), TEXT("_Mat"), [](UTexture* SourceObject)
			{
				UMaterialFactoryNew* Factory = NewObject<UMaterialFactoryNew>();
				Factory->InitialTexture = SourceObject;
				return Factory;
			}
		);
	}

	static void ExecuteResizeTextureSource(const UContentBrowserAssetContextMenuContext* CBContext)
	{
		if (CBContext)
		{
			UE::TextureAssetActions::TextureSource_Resize_WithDialog(CBContext->LoadSelectedObjects<UTexture>());
		}
	}
	
	static void ExecuteResizeToPowerOfTwoTextureSource(const UContentBrowserAssetContextMenuContext* CBContext)
	{
		if (CBContext)
		{
			UE::TextureAssetActions::TextureSource_ResizeToPowerOfTwo_WithDialog(CBContext->LoadSelectedObjects<UTexture>());
		}
	}

	static void Execute8bitTextureSource(const UContentBrowserAssetContextMenuContext* CBContext)
	{
		if (CBContext)
		{
			UE::TextureAssetActions::TextureSource_ConvertTo8bit_WithDialog(CBContext->LoadSelectedObjects<UTexture>());
		}
	}
	
	static void ExecuteJPEGTextureSource(const UContentBrowserAssetContextMenuContext* CBContext)
	{
		if (CBContext)
		{
			UE::TextureAssetActions::TextureSource_JPEG_WithDialog(CBContext->LoadSelectedObjects<UTexture>());
		}
	}

	static void ExecuteConvertToVirtualTexture(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			IVirtualTexturingEditorModule* Module = FModuleManager::Get().GetModulePtr<IVirtualTexturingEditorModule>("VirtualTexturingEditor");
			Module->ConvertVirtualTexturesWithDialog(CBContext->LoadSelectedObjects<UTexture2D>(), false);
		}
	}

	static void ExecuteConvertToRegularTexture(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			IVirtualTexturingEditorModule* Module = FModuleManager::Get().GetModulePtr<IVirtualTexturingEditorModule>("VirtualTexturingEditor");
			Module->ConvertVirtualTexturesWithDialog(CBContext->LoadSelectedObjects<UTexture2D>(), true);
		}
	}

	static void ExecuteFindMaterials(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			TArray<FAssetData> MaterialsUsingTexture;

			// This finds "Material like" objects. It's called material in the UI string but this generally
			// seems more useful
			if ( const FAssetData* TextureAsset = CBContext->GetSingleSelectedAssetOfType(UTexture::StaticClass()) )
			{
				UAssetRegistryHelpers::FindReferencersOfAssetOfClass(TextureAsset->PackageName, { UMaterialInterface::StaticClass(), UMaterialFunction::StaticClass() }, MaterialsUsingTexture);
			}

			if (MaterialsUsingTexture.Num() > 0)
			{
				IAssetTools::Get().SyncBrowserToAssets(MaterialsUsingTexture);
			}
		}
	}

	static TSharedRef<SWidget> MakeTextureSourceActionMenu(const UContentBrowserAssetContextMenuContext* Context)
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		
		{
			const TAttribute<FText> Label = LOCTEXT("Texture_ResizeSourceToPowerOfTwo", "Resize To Power of Two");
			const TAttribute<FText> ToolTip = LOCTEXT("Texture_ResizeSourceToPowerOfTwoTooltip", "Change texture source dimensions to the nearest power of two.");
			const FUIAction UIAction = FUIAction(FExecuteAction::CreateStatic(&ExecuteResizeToPowerOfTwoTextureSource, Context), FCanExecuteAction());
			MenuBuilder.AddMenuEntry(Label, ToolTip, FSlateIcon(), UIAction);
		}

		{
			const TAttribute<FText> Label = LOCTEXT("Texture_ResizeSource", "Reduce Size");
			const TAttribute<FText> ToolTip = LOCTEXT("Texture_ResizeSourceTooltip", "Reduce texture asset size by shrinking the texture source dimensions.");
			const FUIAction UIAction = FUIAction(FExecuteAction::CreateStatic(&ExecuteResizeTextureSource, Context), FCanExecuteAction());
			MenuBuilder.AddMenuEntry(Label, ToolTip, FSlateIcon(), UIAction);
		}
												
		{
			const TAttribute<FText> Label = LOCTEXT("Texture_ConvertTo8bit", "Convert To 8 bit or minimum bit depth");
			const TAttribute<FText> ToolTip = LOCTEXT("Texture_ConvertTo8bitTooltip", "Reduce texture asset size by converting 16/32 bit source data to 8 bit or minimum compatible bit depth.");
			const FUIAction UIAction = FUIAction(FExecuteAction::CreateStatic(&Execute8bitTextureSource, Context), FCanExecuteAction());
			MenuBuilder.AddMenuEntry(Label, ToolTip, FSlateIcon(), UIAction);
		}
						
		{
			const TAttribute<FText> Label = LOCTEXT("Texture2D_JPEGSource", "Compress With JPEG");
			const TAttribute<FText> ToolTip = LOCTEXT("Texture2D_JPEGSourceTooltip", "Reduce texture asset size by compressing source with JPEG.");
			const FUIAction UIAction = FUIAction(FExecuteAction::CreateStatic(&ExecuteJPEGTextureSource, Context), FCanExecuteAction());
			MenuBuilder.AddMenuEntry(Label, ToolTip, FSlateIcon(), UIAction);
		}

		return MenuBuilder.MakeWidget();
	}


	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			{
				UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UTexture::StaticClass())->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
					if (Context && Context->SelectedAssets.Num() > 0)
					{
						FToolMenuSection& InSection = InMenu->FindOrAddSection("GetAssetActions");

						{
							const TAttribute<FText> Label = LOCTEXT("Texture_CreateMaterial", "Create Material");
							const TAttribute<FText> ToolTip = LOCTEXT("Texture_CreateMaterialTooltip", "Creates a new material using this texture.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Material");
							const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateMaterial);
							InSection.AddMenuEntry("Texture_CreateMaterial", Label, ToolTip, Icon, UIAction);
						}
															
						if ( Context->SelectedAssets.Num() == 1 )
						{
							const TAttribute<FText> Label = LOCTEXT("Texture_FindMaterials", "Find Materials Using This");
							const TAttribute<FText> ToolTip = LOCTEXT("Texture_FindMaterialsTooltip", "Finds all materials that use this material in the content browser.");
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Find");
							const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindMaterials);
							InSection.AddMenuEntry("Texture_FindMaterials", Label, ToolTip, Icon, UIAction);
						}
						
						if (Context->bCanBeModified)
						{
							const TAttribute<FText> Label = LOCTEXT("Texture_SourceActions", "Texture Source Actions");
							const TAttribute<FText> ToolTip = LOCTEXT("Texture_SourceActionsTooltip", "Actions that modify the texture source");
							InSection.AddSubMenu("Texture_SourceActions", 
												 Label, 
												 ToolTip, 
												 FNewToolMenuChoice { FOnGetContent::CreateStatic(&MakeTextureSourceActionMenu, Context) },
												 false,
												 FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Texture2D"));
						}
					}
				}));
			}

			{
				// VT actions should only be on Texture2D , not all UTexture
				UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UTexture2D::StaticClass())->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
					if (Context && Context->bCanBeModified && Context->SelectedAssets.Num() > 0)
					{											
						FToolMenuSection& InSection = InMenu->FindOrAddSection("GetAssetActions");

						static const auto CVarVirtualTexturesEnabled = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VirtualTextures"));
						check(CVarVirtualTexturesEnabled);
						
						bool bVTEnabled = !! CVarVirtualTexturesEnabled->GetValueOnAnyThread();
						
						static const auto CVarVirtualTexturesMenuRestricted = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VT.MenuRestricted"));
						check(CVarVirtualTexturesMenuRestricted);
						
						bool bVTMenuRestricted = !! CVarVirtualTexturesMenuRestricted->GetValueOnAnyThread();

						if ( bVTEnabled && ! bVTMenuRestricted )
						{
							const bool bHasVirtualTextures =
								Algo::AnyOf(Context->SelectedAssets, [](const FAssetData& AssetData){ 
									bool VirtualTextured = false;
									AssetData.GetTagValue<bool>("VirtualTextureStreaming", VirtualTextured);
									return VirtualTextured;
								});
								
							const bool bHasNonVirtualTextures =
								Algo::AnyOf(Context->SelectedAssets, [](const FAssetData& AssetData){ 
									bool VirtualTextured = false;
									AssetData.GetTagValue<bool>("VirtualTextureStreaming", VirtualTextured);
									return !VirtualTextured;
								});

							if (bHasVirtualTextures)
							{
								const TAttribute<FText> Label = LOCTEXT("Texture_ConvertToRegular", "Convert VT to Regular Texture");
								const TAttribute<FText> ToolTip = LOCTEXT("Texture_ConvertToRegularTooltip", "Converts this texture to a regular 2D texture if it is a virtual texture.");
								const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Texture2D");
								const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteConvertToRegularTexture);
								InSection.AddMenuEntry("Texture_ConvertToRegular", Label, ToolTip, Icon, UIAction);
							}
						
							if (bHasNonVirtualTextures)
							{
								const TAttribute<FText> Label = LOCTEXT("Texture_ConvertToVT", "Convert to Virtual Texture");
								const TAttribute<FText> ToolTip = LOCTEXT("Texture_ConvertToVTTooltip", "Converts this texture to a virtual texture if it exceeds the specified size.");
								const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Texture2D");
								const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteConvertToVirtualTexture);
								InSection.AddMenuEntry("Texture_ConvertToVT", Label, ToolTip, Icon, UIAction);
							}
						}
					}
				}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
