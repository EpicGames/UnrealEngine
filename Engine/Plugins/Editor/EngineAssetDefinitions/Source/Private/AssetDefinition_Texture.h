// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_Texture.generated.h"

UCLASS()
class UAssetDefinition_Texture : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Texture", "BaseTexture"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(192, 64, 64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTexture::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Texture };
		return Categories;
	}
	virtual bool CanImport() const override { return true; }

	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
