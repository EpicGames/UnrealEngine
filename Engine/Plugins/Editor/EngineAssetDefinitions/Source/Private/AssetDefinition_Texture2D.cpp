// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_Texture2D.h"

#include "AssetToolsModule.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"
#include "Misc/NamePermissionList.h"
#include "Misc/PackageName.h"
#include "Factories/SlateBrushAssetFactory.h"
#include "Slate/SlateBrushAsset.h"
#include "Factories/VolumeTextureFactory.h"
#include "Factories/Texture2DArrayFactory.h"
#include "Engine/VolumeTexture.h"
#include "Engine/Texture2DArray.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Texture2D)

#define LOCTEXT_NAMESPACE "UAssetDefinition_Texture2D"

namespace MenuExtension_Texture2D
{
	void ExecuteCreateSlateBrush(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);
		TArray<UTexture2D*> SelectedTextures = Context->LoadSelectedObjects<UTexture2D>();
		
		const FString DefaultSuffix = TEXT("_Brush");
		
		TArray<UObject*> ObjectsToSync;

		for (UTexture2D* Texture : SelectedTextures)
		{
			// Determine the asset name
			FString Name;
			FString PackageName;
			IAssetTools::Get().CreateUniqueAssetName(Texture->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

			// Create the factory used to generate the asset
			USlateBrushAssetFactory* Factory = NewObject<USlateBrushAssetFactory>();
			Factory->InitialTexture = Texture;

			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
			UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), USlateBrushAsset::StaticClass(), Factory);

			if( NewAsset )
			{
				ObjectsToSync.Add(NewAsset);
			}
		}

		if( ObjectsToSync.Num() > 0 )
		{
			IAssetTools::Get().SyncBrowserToAssets(ObjectsToSync);
		}
	}

	void ExecuteCreateTextureArray(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);
		TArray<UTexture2D*> SelectedTextures = Context->LoadSelectedObjects<UTexture2D>();
		if(SelectedTextures.Num() > 0)
		{
			const FString DefaultSuffix = TEXT("_Array");
			FString Name;
			FString PackagePath;
			IAssetTools::Get().CreateUniqueAssetName(SelectedTextures[0]->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);
		
			// Create the factory used to generate the asset.
			UTexture2DArrayFactory* Factory = NewObject<UTexture2DArrayFactory>();
			Factory->InitialTextures.Empty();

			// Give the selected textures to the factory.
			for (int32 TextureIndex = 0; TextureIndex < SelectedTextures.Num(); ++TextureIndex)
			{
				Factory->InitialTextures.Add(SelectedTextures[TextureIndex]);
			}

			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), UTexture2DArray::StaticClass(), Factory);
			}
		}
	}

	void ExecuteCreateVolumeTexture(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);
		const FString DefaultSuffix = TEXT("_Volume");

		if (Context->SelectedAssets.Num() == 1)
		{
			if (UTexture2D* Object = Cast<UTexture2D>(Context->SelectedAssets[0].GetAsset()))
			{
				// Determine the asset name
				FString Name;
				FString PackagePath;
				IAssetTools::Get().CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

				// Create the factory used to generate the asset
				UVolumeTextureFactory* Factory = NewObject<UVolumeTextureFactory>();
				Factory->InitialTexture = Object;
				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), UVolumeTexture::StaticClass(), Factory);
			}
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UTexture2D::StaticClass())->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
				if (Context && Context->bCanBeModified && Context->SelectedAssets.Num() > 0)
				{
					TSharedRef<FPathPermissionList> ClassPathPermissionList = IAssetTools::Get().GetAssetClassPathPermissionList(EAssetClassAction::CreateAsset);
					FToolMenuSection& InSection = InMenu->FindOrAddSection("GetAssetActions");

					if (ClassPathPermissionList->PassesFilter(USlateBrushAsset::StaticClass()->GetClassPathName().ToString()))
					{
						const TAttribute<FText> Label = LOCTEXT("Texture2D_CreateSlateBrush", "Create Slate Brush");
						const TAttribute<FText> ToolTip = LOCTEXT("Texture2D_CreateSlateBrushToolTip", "Creates a new slate brush using this texture.");
						const FSlateIcon Icon = FSlateIconFinder::FindIconForClass(USlateBrushAsset::StaticClass());
						const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateSlateBrush);

						InSection.AddMenuEntry("Texture2D_CreateSlateBrush", Label, ToolTip, Icon, UIAction);
					}

					if (ClassPathPermissionList->PassesFilter(UTexture2DArray::StaticClass()->GetClassPathName().ToString()))
					{
						static const auto AllowTextureArrayAssetCreationVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AllowTexture2DArrayCreation"));
						if (AllowTextureArrayAssetCreationVar->GetValueOnGameThread() != 0)
						{
							const TAttribute<FText> Label = LOCTEXT("Texture_Texture2DArray", "Create Texture Array");
							const TAttribute<FText> ToolTip = LOCTEXT("Texture_CreateTexture2DArrayTooltip", "Creates a new texture array.");
							const FSlateIcon Icon = FSlateIconFinder::FindIconForClass(UTexture2DArray::StaticClass());
							const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateTextureArray);

							InSection.AddMenuEntry("Texture_Texture2DArray", Label, ToolTip, Icon, UIAction);
						}
					}

					if (Context->SelectedAssets.Num() == 1 && ClassPathPermissionList->PassesFilter(UVolumeTexture::StaticClass()->GetClassPathName().ToString()))
					{
						const TAttribute<FText> Label = LOCTEXT("Texture2D_CreateVolumeTexture", "Create Volume Texture");
						const TAttribute<FText> ToolTip = LOCTEXT("Texture2D_CreateVolumeTextureToolTip", "Creates a new volume texture using this texture.");
						const FSlateIcon Icon = FSlateIconFinder::FindIconForClass(UVolumeTexture::StaticClass());
						const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateVolumeTexture);

						InSection.AddMenuEntry("Texture2D_CreateVolumeTexture", Label, ToolTip, Icon, UIAction);
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
