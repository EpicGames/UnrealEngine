// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2D.h"
#include "AssetDefinition_Texture.h"

#include "AssetDefinition_Texture2D.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_Texture2D : public UAssetDefinition_Texture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Texture2D", "Texture"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTexture2D::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Texture, EAssetCategoryPaths::Basic };
		return Categories;
	}
	// UAssetDefinition End
};
