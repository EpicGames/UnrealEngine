// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2DArray.h"
#include "AssetDefinition_Texture.h"

#include "AssetDefinition_Texture2DArray.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_Texture2DArray : public UAssetDefinition_Texture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Texture2DArray", "Texture 2D Array"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 64, 128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTexture2DArray::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture) };
		return Categories;
	}
	// UAssetDefinition End
};
