// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_TextureCube.h"

#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"
#include "Misc/PackageName.h"
#include "Factories/TextureCubeArrayFactory.h"
#include "Engine/TextureCubeArray.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_TextureCube)

#define LOCTEXT_NAMESPACE "UAssetDefinition_TextureCubeArray"

namespace MenuExtension_TextureCubeArray
{
	static void ExecuteCreateTextureArray(const FToolMenuContext& MenuContext)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			TArray<UTextureCube*> SelectedTextureCubes = Context->LoadSelectedObjects<UTextureCube>();
			if (SelectedTextureCubes.Num() > 0)
			{
				const FString DefaultSuffix = TEXT("_Array");
				FString Name;
				FString PackagePath;
				IAssetTools::Get().CreateUniqueAssetName(SelectedTextureCubes[0]->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

				// Create the factory used to generate the asset.
				UTextureCubeArrayFactory* Factory = NewObject<UTextureCubeArrayFactory>();
				Factory->InitialTextures.Empty();

				// Give the selected textures to the factory.
				for (int32 TextureIndex = 0; TextureIndex < SelectedTextureCubes.Num(); ++TextureIndex)
				{
					Factory->InitialTextures.Add(SelectedTextureCubes[TextureIndex]);
				}

				{
					FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), UTextureCubeArray::StaticClass(), Factory);
				}
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UTextureCube::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				TSharedRef<FPathPermissionList> ClassPathPermissionList = IAssetTools::Get().GetAssetClassPathPermissionList(EAssetClassAction::CreateAsset);
				
				// check that UTextureCubeArray classes are allowed in this project :
				if (ClassPathPermissionList->PassesFilter(UTextureCubeArray::StaticClass()->GetClassPathName().ToString()))
				{
					// share the AllowTexture2DArrayCreation CVar to toggle this rather than make a separate one for cubes
					static const auto AllowTextureArrayAssetCreationVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AllowTexture2DArrayCreation"));
					if (AllowTextureArrayAssetCreationVar->GetValueOnGameThread() != 0)
					{
						const TAttribute<FText> Label = LOCTEXT("Texture_TextureCubeArray", "Create Texture Cube Array");
						const TAttribute<FText> ToolTip = LOCTEXT("Texture_CreateTextureCubeArrayTooltip", "Creates a new texture cube array.");
						const FSlateIcon Icon = FSlateIconFinder::FindIconForClass(UTextureCubeArray::StaticClass());
						const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateTextureArray);

						InSection.AddMenuEntry("Texture_TextureCubeArray", Label, ToolTip, Icon, UIAction);
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
