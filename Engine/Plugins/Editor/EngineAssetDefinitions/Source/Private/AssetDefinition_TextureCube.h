// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureCube.h"
#include "AssetDefinition_Texture.h"

#include "AssetDefinition_TextureCube.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_TextureCube : public UAssetDefinition_Texture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TextureCube", "Texture Cube"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(192,64,128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTextureCube::StaticClass(); }
	// UAssetDefinition End
};
