// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureCubeArray.h"
#include "AssetDefinition_Texture.h"

#include "AssetDefinition_TextureCubeArray.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_TextureCubeArray : public UAssetDefinition_Texture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TextureCubeArray", "Texture Cube Array"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(192, 192, 128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTextureCubeArray::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture, NSLOCTEXT("AssetDefinition", "TextureCubeArray_SubMenu", "Texture Array"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
