// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureLightProfile.h"
#include "AssetDefinition_Texture.h"

#include "AssetDefinition_TextureLightProfile.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_TextureLightProfile : public UAssetDefinition_Texture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TextureLightProfile", "Texture Light Profile"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(192,64,192)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTextureLightProfile::StaticClass(); }
	// UAssetDefinition End
};
