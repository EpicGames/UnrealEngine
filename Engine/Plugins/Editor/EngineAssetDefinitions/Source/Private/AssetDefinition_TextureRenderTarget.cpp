// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_TextureRenderTarget.h"

#include "ContentBrowserMenuContexts.h"
#include "IAssetTools.h"
#include "ToolMenus.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCube.h"
#include "Engine/VolumeTexture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/TextureRenderTargetCube.h"
#include "Engine/TextureRenderTargetVolume.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_TextureRenderTarget)

#define LOCTEXT_NAMESPACE "UAssetDefinition_TextureRenderTarget"

namespace MenuExtension_TextureRenderTarget
{
	static void ExecuteCreateStaticTexture(const FToolMenuContext& MenuContext)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
		{
			TArray<UTextureRenderTarget*> SelectedRenderTargets = Context->LoadSelectedObjects<UTextureRenderTarget>();
			for (UTextureRenderTarget* RenderTarget : SelectedRenderTargets)
			{
				FString Name;
				FString PackageName;
				IAssetTools::Get().CreateUniqueAssetName(RenderTarget->GetOutermost()->GetName(), TEXT("_Tex"), PackageName, Name);

				UObject* NewObj = nullptr;
				UTextureRenderTarget2D* TexRT = Cast<UTextureRenderTarget2D>(RenderTarget);
				UTextureRenderTarget2DArray* TexRT2DArray = Cast<UTextureRenderTarget2DArray>(RenderTarget);
				UTextureRenderTargetCube* TexRTCube = Cast<UTextureRenderTargetCube>(RenderTarget);
				UTextureRenderTargetVolume* TexRTVolume = Cast<UTextureRenderTargetVolume>(RenderTarget);
				if( TexRTCube )
				{
					// create a static cube texture as well as its 6 faces
					NewObj = TexRTCube->ConstructTextureCube( CreatePackage(*PackageName), Name, RenderTarget->GetMaskedFlags() );
				}
				else if (TexRTVolume)
				{
					NewObj = TexRTVolume->ConstructTextureVolume(CreatePackage( *PackageName), Name, RenderTarget->GetMaskedFlags());
				}
				else if (TexRT2DArray)
				{
					NewObj = TexRT2DArray->ConstructTexture2DArray(CreatePackage(*PackageName), Name, RenderTarget->GetMaskedFlags());
				}
				else if( TexRT )
				{
					// create a static 2d texture
					NewObj = TexRT->ConstructTexture2D( CreatePackage(*PackageName), Name, RenderTarget->GetMaskedFlags(), CTF_Default, nullptr );
				}

				if( NewObj )
				{
					// package needs saving
					NewObj->MarkPackageDirty();

					// Notify the asset registry
					FAssetRegistryModule::AssetCreated(NewObj);
				}
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UTextureRenderTarget::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("TextureRenderTarget_CreateStatic", "Create Static Texture");
					const TAttribute<FText> ToolTip = LOCTEXT("TextureRenderTarget_CreateStaticTooltip", "Creates a static texture from the selected render targets.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Texture2D");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateStaticTexture);

					InSection.AddMenuEntry("TextureRenderTarget_CreateStatic", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
