// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureRenderTarget.h"
#include "AssetDefinition_Texture.h"

#include "AssetDefinition_TextureRenderTarget.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_TextureRenderTarget : public UAssetDefinition_Texture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TextureRenderTarget", "Texture Render Target"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,64,64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTextureRenderTarget::StaticClass(); }
	virtual bool CanImport() const override { return false; }
	// UAssetDefinition End
};
