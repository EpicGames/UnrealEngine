// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureRenderTarget2D.h"
#include "AssetDefinition_TextureRenderTarget.h"

#include "AssetDefinition_TextureRenderTarget2D.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_TextureRenderTarget2D : public UAssetDefinition_TextureRenderTarget
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TextureRenderTarget2D", "Render Target"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTextureRenderTarget2D::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture) };
		return Categories;
	}
	// UAssetDefinition End
};
