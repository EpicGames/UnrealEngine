// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureRenderTarget2DArray.h"
#include "AssetDefinition_TextureRenderTarget.h"

#include "AssetDefinition_TextureRenderTarget2DArray.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_TextureRenderTarget2DArray : public UAssetDefinition_TextureRenderTarget
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TextureRenderTarget2DArray", "2D Array Render Target"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTextureRenderTarget2DArray::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture, NSLOCTEXT("AssetDefinition", "TextureRenderTarget2DArray_SubMenu", "Render Target"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
