// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureRenderTargetCube.h"
#include "AssetDefinition_TextureRenderTarget.h"

#include "AssetDefinition_TextureRenderTargetCube.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_TextureRenderTargetCube : public UAssetDefinition_TextureRenderTarget
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TextureRenderTargetCube", "Cube Render Target"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTextureRenderTargetCube::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture, NSLOCTEXT("AssetDefinition", "TextureRenderTargetCube_SubMenu", "Render Target"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
