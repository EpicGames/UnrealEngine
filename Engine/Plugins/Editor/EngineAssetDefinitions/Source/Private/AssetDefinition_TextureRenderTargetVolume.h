// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureRenderTargetVolume.h"
#include "AssetDefinition_TextureRenderTarget.h"

#include "AssetDefinition_TextureRenderTargetVolume.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_TextureRenderTargetVolume : public UAssetDefinition_TextureRenderTarget
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TextureRenderTargetVolume", "Volume Render Target"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTextureRenderTargetVolume::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture, NSLOCTEXT("AssetDefinition", "TextureRenderTargetVolume_SubMenu", "Render Target"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
