// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/ToonProfile.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ToonProfile.generated.h"

UCLASS()
class UAssetDefinition_ToonProfile : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ToonProfile", "Toon Profile"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128, 128, 255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UToonProfile::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
    {
    	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material, NSLOCTEXT("Material", "MaterialAssetSubMenu_Profiles", "Profiles"), ECategoryMenuType::Section) };
    	return Categories;
    }
	// UAssetDefinition End
};
