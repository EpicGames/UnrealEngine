// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/TouchInterface.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_TouchInterface.generated.h"

UCLASS()
class UAssetDefinition_TouchInterface : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TouchInterface", "Touch Interface Setup"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(175, 0, 128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTouchInterface::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Input, NSLOCTEXT("AssetDefinition", "TouchInterface_SubMenu", "Touchscreen"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
