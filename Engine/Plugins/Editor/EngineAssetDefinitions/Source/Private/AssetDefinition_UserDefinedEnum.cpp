// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_UserDefinedEnum.h"
#include "BlueprintEditorModule.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_UserDefinedEnum)

#define LOCTEXT_NAMESPACE "UAssetDefinition_UserDefinedEnum"

FText UAssetDefinition_UserDefinedEnum::GetAssetDescription(const FAssetData& AssetData) const
{
	return AssetData.GetTagValueRef<FText>(GET_MEMBER_NAME_CHECKED(UUserDefinedEnum, EnumDescription));
}

EAssetCommandResult UAssetDefinition_UserDefinedEnum::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>( "Kismet" );

	for (UUserDefinedEnum* UDEnum : OpenArgs.LoadObjects<UUserDefinedEnum>())
	{
		BlueprintEditorModule.CreateUserDefinedEnumEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, UDEnum);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
