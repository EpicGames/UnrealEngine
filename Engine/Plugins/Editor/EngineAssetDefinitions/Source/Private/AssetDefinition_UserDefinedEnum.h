// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Engine/UserDefinedEnum.h"

#include "AssetDefinition_UserDefinedEnum.generated.h"

UCLASS()
class UAssetDefinition_UserDefinedEnum : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Enum", "Enumeration"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 200, 200)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UUserDefinedEnum::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Blueprint, NSLOCTEXT("AssetDefinition", "Enum", "Composite Variables"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override
	{
		return FAssetSupportResponse::NotSupported();
	}
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
