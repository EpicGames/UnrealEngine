// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_UserDefinedStruct.h"

#include "BlueprintEditorModule.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_UserDefinedStruct)

#define LOCTEXT_NAMESPACE "UAssetDefinition_UserDefinedStruct"

EAssetCommandResult UAssetDefinition_UserDefinedStruct::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (FStructureEditorUtils::UserDefinedStructEnabled())
	{
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>( "Kismet" );

		for (UUserDefinedStruct* UDStruct : OpenArgs.LoadObjects<UUserDefinedStruct>())
		{
			BlueprintEditorModule.CreateUserDefinedStructEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, UDStruct);
		}
	}

	return EAssetCommandResult::Handled;
}

FText UAssetDefinition_UserDefinedStruct::GetAssetDescription(const FAssetData& AssetData) const
{
	FString Description = AssetData.GetTagValueRef<FString>("Tooltip");
	if (!Description.IsEmpty())
	{
		Description.ReplaceInline(TEXT("\\n"), TEXT("\n"));
		return FText::FromString(MoveTemp(Description));
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
