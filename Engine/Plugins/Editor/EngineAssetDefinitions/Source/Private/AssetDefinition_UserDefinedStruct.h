// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/UserDefinedStruct.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_UserDefinedStruct.generated.h"

UCLASS()
class UAssetDefinition_UserDefinedStruct : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Struct", "Structure"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(103, 206, 218)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UUserDefinedStruct::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Blueprint, NSLOCTEXT("AssetDefinition", "Struct", "Composite Variables"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override
    {
    	return FAssetSupportResponse::NotSupported();
    }
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
