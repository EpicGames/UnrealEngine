// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorField/VectorField.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_VectorField.generated.h"

UCLASS()
class UAssetDefinition_VectorField : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_VectorField", "Vector Field"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(200,128,128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UVectorField::StaticClass(); }
	// UAssetDefinition End
};
