// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorField/VectorFieldAnimated.h"
#include "AssetDefinition_VectorField.h"

#include "AssetDefinition_VectorFieldAnimated.generated.h"

UCLASS()
class UAssetDefinition_VectorFieldAnimated : public UAssetDefinition_VectorField
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_VectorFieldAnimated", "Animated Vector Field"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UVectorFieldAnimated::StaticClass(); }
	// UAssetDefinition End
};
