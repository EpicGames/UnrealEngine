// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorField/VectorFieldStatic.h"
#include "AssetDefinition_VectorField.h"

#include "AssetDefinition_VectorFieldStatic.generated.h"

UCLASS()
class UAssetDefinition_VectorFieldStatic : public UAssetDefinition_VectorField
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_VectorFieldStatic", "Static Vector Field"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UVectorFieldStatic::StaticClass(); }
	virtual bool CanImport() const override { return true; }
	// UAssetDefinition End
};
