// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition_Texture.h"
#include "Engine/VolumeTexture.h"

#include "AssetDefinition_VolumeTexture.generated.h"

UCLASS(MinimalAPI)
class UAssetDefinition_VolumeTexture : public UAssetDefinition_Texture
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_VolumeTexture", "Volume Texture"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(192, 128, 64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UVolumeTexture::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture) };
		return Categories;
	}
	// UAssetDefinition End
};
