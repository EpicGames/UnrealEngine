// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_World.h"

#include "AssetToolsModule.h"
#include "FileHelpers.h"
#include "HAL/FileManager.h"
#include "JsonObjectGraph/Stringify.h"
#include "Misc/PackageName.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "ThumbnailRendering/WorldThumbnailInfo.h"
#include "Misc/MessageDialog.h"
#include "Engine/Level.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_World)

#define LOCTEXT_NAMESPACE "UAssetDefinition_World"

TConstArrayView<FAssetCategoryPath> UAssetDefinition_World::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Basic),
			FAssetCategoryPath(EAssetCategoryPaths::World, LOCTEXT("AssetDefinition_WorldSubMenu", "World"), ECategoryMenuType::Section)
		};
	return Categories;
}

TArray<FAssetData> UAssetDefinition_World::PrepareToActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	TArray<FAssetData> AssetsToOpen;
	if (ActivateArgs.Assets.Num())
	{
		const FAssetData& AssetData = ActivateArgs.Assets[0];

		// If there are any unsaved changes to the current level, see if the user wants to save those first
		// If they do not wish to save, then we will bail out of opening this asset.
		constexpr bool bPromptUserToSave = true;
		constexpr bool bSaveMapPackages = true;
		constexpr bool bSaveContentPackages = true;
		if (FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages))
		{
			// Validate that Asset was saved or isn't loaded meaning it can be loaded
			const bool bLoad = false;
			if (UWorld* World = Cast<UWorld>(AssetData.FastGetAsset(bLoad)); !World || !World->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				AssetsToOpen.Add(AssetData);
			}
			else
			{
				FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("CannotOpenNewlyCreatedMapWithoutSaving", "The level you are trying to open needs to be saved first."));
			}
		}
	}

	return MoveTemp(AssetsToOpen);
}

EAssetCommandResult UAssetDefinition_World::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (const FAssetData& WorldAsset : OpenArgs.Assets)
	{
		UWorld* World = Cast<UWorld>(WorldAsset.GetAsset());
		if (World != nullptr && 
			ensureMsgf(World->GetPackage(), TEXT("World(%s) is not in a package and cannot be opened"), *World->GetFullName()) && 
			ensureMsgf(!World->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated), TEXT("World(%s) is unsaved and cannot be opened"), *World->GetFullName()))
		{
			const FString FileToOpen = FPackageName::LongPackageNameToFilename(World->GetOutermost()->GetName(), FPackageName::GetMapPackageExtension());
			const bool bLoadAsTemplate = false;
			const bool bShowProgress = true;
			FEditorFileUtils::LoadMap(FileToOpen, bLoadAsTemplate, bShowProgress);

			// We can only edit one world at a time... so just break after the first valid world to load
			return EAssetCommandResult::Handled;
		}
	}

	return EAssetCommandResult::Unhandled;
}

EAssetCommandResult UAssetDefinition_World::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	if (DiffArgs.OldAsset == nullptr && DiffArgs.NewAsset == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}

	// The caller has loaded our assets and classified them, create a useful text representation for display:
	const FUtf8String OldAssetJSON = DiffArgs.OldAsset ? UE::JsonObjectGraph::Stringify({DiffArgs.OldAsset->GetPackage()}) : FUtf8String();
	const FUtf8String NewAssetJSON = DiffArgs.NewAsset ? UE::JsonObjectGraph::Stringify({DiffArgs.NewAsset->GetPackage()}) : FUtf8String();

	// Write the JSON to a file, so the TextDiffTool can consume it:
	const auto GetFilename = [](UObject* Asset, const FString& Revision)
	{
		static int32 ID = 0; // ensure subsequent diffs within a session don't stomp eachother, matching UAssetToolsImpl::DumpAssetToTempFile
		FString AssetName = Asset? Asset->GetName() : TEXT("empty");
		FString RelTempFileName = FString::Printf(TEXT("%sJsonDiff%s-%s-%d.txt"), *FPaths::DiffDir(), *AssetName, *Revision, ID++);
		FString AbsoluteTempFileName = FPaths::ConvertRelativePathToFull(RelTempFileName);
		return AbsoluteTempFileName ;
	};

	const FString OldFilename = GetFilename(DiffArgs.OldAsset, DiffArgs.OldRevision.Revision);
	const FString NewFilename = GetFilename(DiffArgs.NewAsset, DiffArgs.NewRevision.Revision);

	const auto WriteUtf8StringToFile = [](const FString& Filename, const FUtf8String& String)
	{
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*Filename));
		if(FileArchive)
		{
			FileArchive->Serialize((void*)String.GetCharArray().GetData(), FMath::Max(String.GetCharArray().Num() - 1, 0));
			// FileArchive closed by ~FArchiveFileWriterGeneric
			return true;
		}
		return false;
	};
	if( !WriteUtf8StringToFile(OldFilename, OldAssetJSON) ||
		!WriteUtf8StringToFile(NewFilename, NewAssetJSON) )
	{
		// we failed to write the files - we won't be able to perform a meaningful diff:
		return EAssetCommandResult::Unhandled;
	}
	
	// launch external diff process, ala UAssetDefinitionDefault::PerformAssetDiff
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	const FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;
	AssetTools.CreateDiffProcess(DiffCommand, OldFilename, NewFilename);

	return EAssetCommandResult::Handled;
}

UThumbnailInfo* UAssetDefinition_World::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), UWorldThumbnailInfo::StaticClass());
}

FAssetSupportResponse UAssetDefinition_World::CanRename(const FAssetData& InAsset) const
{
	if (IsPartitionWorldInUse(InAsset))
	{
		return FAssetSupportResponse::Error(LOCTEXT("CanNotRenameWorldInUse", "Cannot rename a partition world while it is used."));
	}
	
	return FAssetSupportResponse::Supported();
}

FAssetSupportResponse UAssetDefinition_World::CanDuplicate(const FAssetData& InAsset) const
{
	if (IsPartitionWorldInUse(InAsset))
	{
		return FAssetSupportResponse::Error(LOCTEXT("CanNotDuplicateWorldInUse", "Cannot duplicate a partition world while it is used."));
	}
	
	return FAssetSupportResponse::Supported();
}

bool UAssetDefinition_World::IsPartitionWorldInUse(const FAssetData& InAsset) const
{
	if (ULevel::GetIsLevelPartitionedFromAsset(InAsset))
	{
		for (const FWorldContext& WorldContext : GEditor->GetWorldContexts())
		{
			if (const UWorld* World = WorldContext.World(); WorldContext.World() && InAsset.PackageName == World->GetPackage()->GetFName())
			{
				return true;
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
