// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FEngineAssetDefinitionsModule"

class FEngineAssetDefinitionsModule : public IModuleInterface
{
public:

	// IModuleInterface interface
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FEngineAssetDefinitionsModule, EngineAssetDefinitions);
