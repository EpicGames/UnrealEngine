// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_Material.h"
#include "MaterialEditorModule.h"
#include "BlueprintEditor.h"
#include "DiffTool/Widgets/SMaterialDiff.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Material)

EAssetCommandResult UAssetDefinition_Material::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UMaterial* Material : OpenArgs.LoadObjects<UMaterial>())
	{
		IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>( "MaterialEditor" );
		MaterialEditorModule->CreateMaterialEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Material);
	}

	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_Material::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	UMaterial* OldMaterial = Cast<UMaterial>(DiffArgs.OldAsset);
	UMaterial* NewMaterial = Cast<UMaterial>(DiffArgs.NewAsset);

	if (OldMaterial && NewMaterial)
	{
		/** We create transient graphs so that the material asset is not dirtied by graph/package modifications */
		UMaterialGraph* OldMaterialGraph = CastChecked<UMaterialGraph>(FBlueprintEditorUtils::CreateNewGraph(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UMaterialGraph::StaticClass()), UMaterialGraph::StaticClass(), UMaterialGraphSchema::StaticClass()));
		UMaterialGraph* NewMaterialGraph = CastChecked<UMaterialGraph>(FBlueprintEditorUtils::CreateNewGraph(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UMaterialGraph::StaticClass()), UMaterialGraph::StaticClass(), UMaterialGraphSchema::StaticClass()));
		OldMaterialGraph->Material = OldMaterial;
		OldMaterialGraph->MaterialFunction = nullptr;

		NewMaterialGraph->Material = NewMaterial;
		NewMaterialGraph->MaterialFunction = nullptr;

		OldMaterialGraph->RebuildGraph();
		NewMaterialGraph->RebuildGraph();

		SMaterialDiff::CreateDiffWindow(OldMaterialGraph, NewMaterialGraph, DiffArgs.OldRevision, DiffArgs.NewRevision, GetAssetClass().Get());
		return EAssetCommandResult::Handled;
	}

	return EAssetCommandResult::Unhandled;
}
