// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Material/AssetDefinition_MaterialInterface.h"
#include "Materials/Material.h"

#include "AssetDefinition_Material.generated.h"

class UAssetDefinition_MaterialInterface;

UCLASS()
class UAssetDefinition_Material : public UAssetDefinition_MaterialInterface
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Material", "Material"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterial::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Material, EAssetCategoryPaths::Basic };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	// UAssetDefinition End
};
