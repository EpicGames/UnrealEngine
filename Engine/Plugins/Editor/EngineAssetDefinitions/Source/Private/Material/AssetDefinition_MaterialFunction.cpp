// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MaterialFunction.h"

#include "AssetRegistry/AssetIdentifier.h"
#include "ContentBrowserMenuContexts.h"
#include "Factories/MaterialFunctionInstanceFactory.h"
#include "ToolMenus.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "IAssetTools.h"
#include "MaterialEditorModule.h"
#include "Misc/PackageName.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "BlueprintEditor.h"
#include "DiffTool/Widgets/SMaterialDiff.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MaterialFunction)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

const FName MaterialFunctionClass = FName(TEXT("MaterialFunction"));
const FName MaterialFunctionInstanceClass = FName(TEXT("MaterialFunctionInstance"));
const FName MaterialFunctionUsageTag = FName(TEXT("MaterialFunctionUsage"));
const FString LayerCompareString = (TEXT("MaterialLayer"));
const FString BlendCompareString = (TEXT("MaterialLayerBlend"));

EAssetCommandResult UAssetDefinition_MaterialFunction::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UMaterialFunction* Function : OpenArgs.LoadObjects<UMaterialFunction>())
	{
		IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>( "MaterialEditor" );
		MaterialEditorModule->CreateMaterialEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Function);
	}

	return EAssetCommandResult::Handled;
}

UThumbnailInfo* UAssetDefinition_MaterialFunction::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), USceneThumbnailInfoWithPrimitive::StaticClass());
}

EAssetCommandResult UAssetDefinition_MaterialFunction::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	UMaterialFunction* OldMaterialFunction = Cast<UMaterialFunction>(DiffArgs.OldAsset);
	UMaterialFunction* NewMaterialFunction = Cast<UMaterialFunction>(DiffArgs.NewAsset);

	if (OldMaterialFunction && NewMaterialFunction)
	{
		/** We create transient graphs so that the material asset is not dirtied by graph/package modifications */
		UMaterialGraph* OldMaterialGraph = CastChecked<UMaterialGraph>(FBlueprintEditorUtils::CreateNewGraph(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UMaterialGraph::StaticClass()), UMaterialGraph::StaticClass(), UMaterialGraphSchema::StaticClass()));
		UMaterialGraph* NewMaterialGraph = CastChecked<UMaterialGraph>(FBlueprintEditorUtils::CreateNewGraph(GetTransientPackage(), MakeUniqueObjectName(GetTransientPackage(), UMaterialGraph::StaticClass()), UMaterialGraph::StaticClass(), UMaterialGraphSchema::StaticClass()));

		OldMaterialGraph->MaterialFunction = OldMaterialFunction;
		OldMaterialGraph->Material = NewObject<UMaterial>();
		{
			FArchiveUObject DummyArchive;
			// Hack: serialize the new material with an archive that does nothing so that its material resources are created
			OldMaterialGraph->Material->Serialize(DummyArchive);
		}
		OldMaterialGraph->Material->SetShadingModel(MSM_Unlit);
		OldMaterialGraph->Material->AssignExpressionCollection(OldMaterialGraph->MaterialFunction->GetExpressionCollection());

		NewMaterialGraph->MaterialFunction = NewMaterialFunction;
		NewMaterialGraph->Material = NewObject<UMaterial>();
		{
			FArchiveUObject DummyArchive;
			// Hack: serialize the new material with an archive that does nothing so that its material resources are created
			NewMaterialGraph->Material->Serialize(DummyArchive);
		}
		NewMaterialGraph->Material->SetShadingModel(MSM_Unlit);
		NewMaterialGraph->Material->AssignExpressionCollection(NewMaterialGraph->MaterialFunction->GetExpressionCollection());

		OldMaterialGraph->RebuildGraph();
		NewMaterialGraph->RebuildGraph();

		SMaterialDiff::CreateDiffWindow(OldMaterialGraph, NewMaterialGraph, DiffArgs.OldRevision, DiffArgs.NewRevision, GetAssetClass().Get());
		return EAssetCommandResult::Handled;
	}

	return EAssetCommandResult::Unhandled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_MaterialFunction
{
	void ExecuteFindMaterials(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);
		
		if (FEditorDelegates::OnOpenReferenceViewer.IsBound())
		{
			// TArray that will be send to the ReferenceViewer for display
			TArray<FAssetIdentifier> AssetIdentifiers;

			// Iterate over all selected UMaterialFunctionInterface instances
			for (UMaterialFunctionInterface* MaterialFunction : CBContext->LoadSelectedObjects<UMaterialFunctionInterface>())
			{
				// Construct FAssetIdentifier and add to TArray
				const FName AssetName(*FPackageName::ObjectPathToPackageName(GetPathNameSafe(MaterialFunction)));
				AssetIdentifiers.Add(FAssetIdentifier(AssetName));
			}
			
			// Call ReferenceViewer
			FReferenceViewerParams ReferenceViewerParams;
			ReferenceViewerParams.bShowDependencies = false;
			ReferenceViewerParams.FixAndHideSearchDepthLimit = 1;
			ReferenceViewerParams.bShowShowReferencesOptions = false;
			ReferenceViewerParams.bShowShowSearchableNames = false;
			FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, ReferenceViewerParams);
		}
	}

	void ExecuteNewMFI(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);

		IAssetTools::Get().CreateAssetsFrom<UMaterialFunctionInterface>(
			CBContext->LoadSelectedObjects<UMaterialFunctionInterface>(), UMaterialFunctionInstance::StaticClass(), TEXT("_Inst"), [](UMaterialFunctionInterface* SourceObject)
			{
				UMaterialFunctionInstanceFactory* Factory = NewObject<UMaterialFunctionInstanceFactory>();
				Factory->InitialParent = SourceObject;
				return Factory;
			}
		);
	}
	
	FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UMaterialFunction::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = NSLOCTEXT("AssetTypeActions", "Material_NewMFI", "Create Function Instance");
					const TAttribute<FText> ToolTip = NSLOCTEXT("AssetTypeActions", "Material_NewMFITooltip", "Creates a parameterized function using this function as a base.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MaterialInstanceActor");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewMFI);

					InSection.AddMenuEntry("MaterialFunction_NewMFI", Label, ToolTip, Icon, UIAction);
				}
			}));
			
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (FEditorDelegates::OnOpenReferenceViewer.IsBound())
				{
					const TAttribute<FText> Label = LOCTEXT("MaterialFunction_FindMaterials", "Find Materials Using This");
					const TAttribute<FText> ToolTip = LOCTEXT("MaterialFunction_FindMaterialsTooltip", "Finds the materials that reference this material function and visually displays them with the light version of the Reference Viewer.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.GenericFind");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindMaterials);

					InSection.AddMenuEntry("MaterialFunction_FindMaterials", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

namespace MenuExtension_MaterialFunctionMaterialLayer
{
	void ExecuteNewMFI(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);

		IAssetTools::Get().CreateAssetsFrom<UMaterialFunctionInterface>(
			CBContext->LoadSelectedObjects<UMaterialFunctionInterface>(), UMaterialFunctionMaterialLayerInstance::StaticClass(), TEXT("_Inst"), [](UMaterialFunctionInterface* SourceObject)
			{
				UMaterialFunctionMaterialLayerInstanceFactory* Factory = NewObject<UMaterialFunctionMaterialLayerInstanceFactory>();
				Factory->InitialParent = SourceObject;
				return Factory;
			}
		);
	}
	
	FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			TArray<UToolMenu*> Menus = UE::ContentBrowser::ExtendToolMenu_AssetContextMenus({ UMaterialFunctionMaterialLayer::StaticClass(), UMaterialFunctionMaterialLayerInstance::StaticClass() });
			
			for (UToolMenu* Menu : Menus)
			{
				FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					{
						const TAttribute<FText> Label = NSLOCTEXT("AssetTypeActions", "Material_NewMLI", "Create Layer Instance");
						const TAttribute<FText> ToolTip = NSLOCTEXT("AssetTypeActions", "Material_NewMFITooltip", "Creates a parameterized function using this function as a base.");
						const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MaterialInstanceActor");
						const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewMFI);

						InSection.AddMenuEntry("MaterialFunction_NewMFI", Label, ToolTip, Icon, UIAction);
					}
				}));
			}
		}));
	});
}

namespace MenuExtension_MaterialFunctionMaterialLayerBlend
{
	void ExecuteNewMFI(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);

		IAssetTools::Get().CreateAssetsFrom<UMaterialFunctionInterface>(
			CBContext->LoadSelectedObjects<UMaterialFunctionInterface>(), UMaterialFunctionMaterialLayerBlendInstance::StaticClass(), TEXT("_Inst"), [](UMaterialFunctionInterface* SourceObject)
			{
				UMaterialFunctionMaterialLayerBlendInstanceFactory* Factory = NewObject<UMaterialFunctionMaterialLayerBlendInstanceFactory>();
				Factory->InitialParent = SourceObject;
				return Factory;
			}
		);
	}
	
	FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			TArray<UToolMenu*> Menus = UE::ContentBrowser::ExtendToolMenu_AssetContextMenus({ UMaterialFunctionMaterialLayerBlend::StaticClass(), UMaterialFunctionMaterialLayerBlendInstance::StaticClass() });

			for (UToolMenu* Menu : Menus)
			{
				FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					{
						const TAttribute<FText> Label = NSLOCTEXT("AssetTypeActions", "Material_NewMBI", "Create Blend Instance");
						const TAttribute<FText> ToolTip = NSLOCTEXT("AssetTypeActions", "Material_NewMFITooltip", "Creates a parameterized function using this function as a base.");
						const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MaterialInstanceActor");
						const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewMFI);

						InSection.AddMenuEntry("MaterialFunction_NewMFI", Label, ToolTip, Icon, UIAction);
					}
				}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
