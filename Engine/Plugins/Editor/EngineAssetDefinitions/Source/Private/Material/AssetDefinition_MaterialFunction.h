// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialFunctionMaterialLayer.h"
#include "Materials/MaterialFunctionMaterialLayerBlend.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_MaterialFunction.generated.h"

UCLASS()
class UAssetDefinition_MaterialFunction : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MaterialFunction", "Material Function"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0,175,175)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialFunction::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material, NSLOCTEXT("Material", "MaterialAssetSubMenu_Advanced", "Advanced"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAsset) const override;
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_MaterialFunctionMaterialLayer : public UAssetDefinition_MaterialFunction
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MaterialFunctionMaterialLayer", "Material Layer"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialFunctionMaterialLayer::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material, NSLOCTEXT("Material", "MaterialAssetSubMenu_Layers", "Layers"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_MaterialFunctionLayerBlend : public UAssetDefinition_MaterialFunction
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MaterialFunctionMaterialLayerBlend", "Material Layer Blend"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialFunctionMaterialLayerBlend::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material, NSLOCTEXT("Material", "MaterialAssetSubMenu_Layers", "Layers"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
