// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MaterialFunctionInstance.h"

#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "IAssetTools.h"
#include "MaterialEditorModule.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MaterialFunctionInstance)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_MaterialFunctionInstance::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UMaterialFunctionInstance* MFI : OpenArgs.LoadObjects<UMaterialFunctionInstance>())
	{
		IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>( "MaterialEditor" );
		MaterialEditorModule->CreateMaterialInstanceEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, MFI);
	}

	return EAssetCommandResult::Handled;
}

UThumbnailInfo* UAssetDefinition_MaterialFunctionInstance::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), USceneThumbnailInfoWithPrimitive::StaticClass());
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_MaterialFunctionInstance
{
	void ExecuteFindParent(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);

		TArray<UObject*> ObjectsToSyncTo;
		for (UMaterialFunctionInstance* MFI : CBContext->LoadSelectedObjects<UMaterialFunctionInstance>())
		{
			if ( MFI->Parent )
			{
				ObjectsToSyncTo.AddUnique( MFI->Parent );
			}
		}

		// Sync the respective browser to the valid parents
		if ( ObjectsToSyncTo.Num() > 0 )
		{
			IAssetTools::Get().SyncBrowserToAssets(ObjectsToSyncTo);
		}
	}
	
	FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UMaterialFunctionInstance::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("MaterialFunctionInstance_FindParent", "Find Parent");
					const TAttribute<FText> ToolTip = LOCTEXT("MaterialFunctionInstance_FindParentTooltip", "Finds the function this instance is based on in the content browser.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.GenericFind");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindParent);

					InSection.AddMenuEntry("MaterialFunctionInstance_FindParent", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
