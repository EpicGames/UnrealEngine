// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Material/AssetDefinition_MaterialFunction.h"
#include "AssetDefinition_MaterialFunctionInstance.generated.h"

enum class EAssetCommandResult : uint8;
struct FAssetOpenArgs;

UCLASS()
class UAssetDefinition_MaterialFunctionInstance : public UAssetDefinition_MaterialFunction
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "FAssetTypeActions_MaterialFunctionInstance", "Material Function Instance"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialFunctionInstance::StaticClass(); }
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAsset) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_MaterialFunctionLayerInstance : public UAssetDefinition_MaterialFunctionInstance
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MaterialFunctionMaterialLayerInstance", "Material Layer Instance"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialFunctionMaterialLayerInstance::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material, NSLOCTEXT("Material", "MaterialAssetSubMenu_Layers", "Layers"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_MaterialFunctionLayerBlendInstance : public UAssetDefinition_MaterialFunctionInstance
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MaterialFunctionMaterialLayerBlendInstance", "Material Layer Blend Instance"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialFunctionMaterialLayerBlendInstance::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material, NSLOCTEXT("Material", "MaterialAssetSubMenu_Layers", "Layers"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
