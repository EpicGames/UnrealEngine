// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MaterialInstanceConstant.h"

#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"
#include "IAssetTools.h"
#include "MaterialEditorModule.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "DiffTool/Widgets/SMaterialDiff.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MaterialInstanceConstant)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_MaterialInstanceConstant::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UMaterialInstanceConstant* MIC : OpenArgs.LoadObjects<UMaterialInstanceConstant>())
   	{
		IMaterialEditorModule* MaterialEditorModule = &FModuleManager::LoadModuleChecked<IMaterialEditorModule>( "MaterialEditor" );
		MaterialEditorModule->CreateMaterialInstanceEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, MIC);
	}

	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_MaterialInstanceConstant::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	UMaterialInstance* OldMaterialInstance = Cast<UMaterialInstance>(DiffArgs.OldAsset);
	UMaterialInstance* NewMaterialInstance = Cast<UMaterialInstance>(DiffArgs.NewAsset);

	if (OldMaterialInstance && NewMaterialInstance)
	{
		SMaterialDiff::CreateDiffWindow(OldMaterialInstance, NewMaterialInstance, DiffArgs.OldRevision, DiffArgs.NewRevision, GetAssetClass().Get());

		return EAssetCommandResult::Handled;
	}

	return EAssetCommandResult::Unhandled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace
{
	void ExecuteFindParent(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);

		TArray<UObject*> ObjectsToSyncTo;
		for (UMaterialInstanceConstant* MIC : CBContext->LoadSelectedObjects<UMaterialInstanceConstant>())
		{
			if ( MIC->Parent )
			{
				ObjectsToSyncTo.AddUnique( MIC->Parent );
			}
		}

		// Sync the respective browser to the valid parents
		if ( ObjectsToSyncTo.Num() > 0 )
		{
			IAssetTools::Get().SyncBrowserToAssets(ObjectsToSyncTo);
		}
	}
	
	FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UMaterialInstanceConstant::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("MaterialInstanceConstant_FindParent", "Find Parent");
					const TAttribute<FText> ToolTip = LOCTEXT("MaterialInstanceConstant_FindParentTooltip", "Finds the material this instance is based on in the content browser.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.GenericFind");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindParent);

					InSection.AddMenuEntry("MaterialInstanceConstant_FindParent", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
