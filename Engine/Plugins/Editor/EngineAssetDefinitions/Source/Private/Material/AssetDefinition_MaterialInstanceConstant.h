// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Material/AssetDefinition_MaterialInterface.h"
#include "Materials/MaterialInstanceConstant.h"

#include "AssetDefinition_MaterialInstanceConstant.generated.h"

class UAssetDefinition_MaterialInterface;

enum class EAssetCommandResult : uint8;
struct FAssetOpenArgs;

UCLASS()
class UAssetDefinition_MaterialInstanceConstant : public UAssetDefinition_MaterialInterface
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MaterialInstanceConstant", "Material Instance"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0,128,0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialInstanceConstant::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material), FAssetCategoryPath(EAssetCategoryPaths::Basic) };
		return Categories;
	}
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	virtual bool CanImport() const override { return true; }
	// UAssetDefinition End
};
