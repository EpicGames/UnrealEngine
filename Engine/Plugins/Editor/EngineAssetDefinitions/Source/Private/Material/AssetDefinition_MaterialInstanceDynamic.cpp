// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MaterialInstanceDynamic.h"
#include "Toolkits/SimpleAssetEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MaterialInstanceDynamic)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_MaterialInstanceDynamic::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TSharedRef<FSimpleAssetEditor> Editor = FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, OpenArgs.LoadObjects<UObject>());
	Editor->SetPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateStatic([]() -> bool
	{
		return true;
	}));

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
