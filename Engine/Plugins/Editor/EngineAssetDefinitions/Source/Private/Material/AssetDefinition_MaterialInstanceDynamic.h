// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Material/AssetDefinition_MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "AssetDefinition_MaterialInstanceDynamic.generated.h"

class UAssetDefinition_MaterialInterface;

enum class EAssetCommandResult : uint8;
struct FAssetOpenArgs;

UCLASS()
class UAssetDefinition_MaterialInstanceDynamic : public UAssetDefinition_MaterialInterface
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MaterialInstanceDynamic", "Material Instance Dynamic"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialInstanceDynamic::StaticClass(); }
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
