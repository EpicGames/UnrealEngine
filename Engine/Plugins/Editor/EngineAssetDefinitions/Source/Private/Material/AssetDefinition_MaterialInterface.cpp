// Copyright Epic Games, Inc. All Rights Reserved.

#include "Material/AssetDefinition_MaterialInterface.h"

#include "ContentBrowserMenuContexts.h"
#include "Materials/Material.h"
#include "ToolMenus.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "Materials/MaterialInstanceConstant.h"
#include "IAssetTools.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MaterialInterface)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UThumbnailInfo* UAssetDefinition_MaterialInterface::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(InAsset.GetAsset()))
	{
		if (USceneThumbnailInfoWithPrimitive* ThumbnailInfo = UE::Editor::FindOrCreateThumbnailInfo<USceneThumbnailInfoWithPrimitive>(MaterialInterface))
		{
			if (MaterialInterface->GetUsageByFlag(MATUSAGE_ParticleSprites))
			{
				ThumbnailInfo->DefaultPrimitiveType = TPT_Plane;
			}

			return ThumbnailInfo;
		}
	}

	return nullptr;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_MaterialInterface
{
	static void ExecuteNewMIC(const FToolMenuContext& MenuContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext);

		IAssetTools::Get().CreateAssetsFrom<UMaterialInterface>(
			CBContext->LoadSelectedObjects<UMaterialInterface>(), UMaterialInstanceConstant::StaticClass(), TEXT("_Inst"), [](UMaterialInterface* SourceObject)
			{
				UMaterialInstanceConstantFactoryNew* Factory = NewObject<UMaterialInstanceConstantFactoryNew>();
				Factory->InitialParent = SourceObject;
				return Factory;
			}
		);
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UMaterialInterface::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("Material_NewMIC", "Create Material Instance");
					const TAttribute<FText> ToolTip = LOCTEXT("Material_NewMICTooltip", "Creates a parameterized material using this material as a base.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MaterialInstanceActor");
					const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewMIC);

					InSection.AddMenuEntry("Material_NewMIC", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}


#undef LOCTEXT_NAMESPACE
