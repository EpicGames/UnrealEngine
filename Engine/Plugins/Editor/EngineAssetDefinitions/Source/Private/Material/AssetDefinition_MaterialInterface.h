// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialInterface.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_MaterialInterface.generated.h"

UCLASS()
class UAssetDefinition_MaterialInterface : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MaterialInterface", "Material Interface"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(64,192,64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialInterface::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Material / NSLOCTEXT("Material", "MaterialAssetSubMenu_Advanced", "Advanced") };
		return Categories;
	}
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAsset) const override;
	// UAssetDefinition End
};
