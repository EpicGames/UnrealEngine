// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialParameterCollection.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_MaterialParameterCollection.generated.h"

UCLASS()
class UAssetDefinition_MaterialParameterCollection : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MaterialParameterCollection", "Material Parameter Collection"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 192, 0)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMaterialParameterCollection::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Material) };
		return Categories;
	}
	// UAssetDefinition End
};
