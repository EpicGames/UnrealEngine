// Copyright Epic Games, Inc. All Rights Reserved.

#include "Script/AssetDefinition_Blueprint.h"

#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "IAssetTools.h"
#include "ToolMenus.h"
#include "Factories/BlueprintFactory.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IContentBrowserSingleton.h"
#include "Algo/AllOf.h"
#include "Logging/MessageLog.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "BlueprintEditor.h"
#include "MergeUtils.h"
#include "SBlueprintDiff.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Blueprint)

#define LOCTEXT_NAMESPACE "UAssetDefinition_Blueprint"

FText UAssetDefinition_Blueprint::GetAssetDisplayName(const FAssetData& AssetData) const
{
	FString OutBlueprintType;
	if (AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintType), OutBlueprintType))
	{
		if (OutBlueprintType == TEXT("BPTYPE_Interface"))
		{
			return LOCTEXT("AssetTypeActions_BlueprintInterface", "Blueprint Interface");
		}
		else if (OutBlueprintType == TEXT("BPTYPE_MacroLibrary"))
        {
			return LOCTEXT("AssetTypeActions_BlueprintMacroLibrary", "Blueprint Macro Library");
        }
		else if (OutBlueprintType == TEXT("BPTYPE_FunctionLibrary"))
		{
			return LOCTEXT("AssetTypeActions_BlueprintFunctionLibrary", "Blueprint Function Library");
		}
	}

	return Super::GetAssetDisplayName(AssetData);
}

FText UAssetDefinition_Blueprint::GetAssetDescription(const FAssetData& AssetData) const
{
	FString Description = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UBlueprint, BlueprintDescription));
	if (!Description.IsEmpty())
	{
		Description.ReplaceInline(TEXT("\\n"), TEXT("\n"));
		return FText::FromString(MoveTemp(Description));
	}

	return FText::GetEmpty();
}

EAssetCommandResult UAssetDefinition_Blueprint::Merge(const FAssetAutomaticMergeArgs& MergeArgs) const
{
	UBlueprint* AsBlueprint = CastChecked<UBlueprint>(MergeArgs.LocalAsset);
	
	if (FBlueprintEditorUtils::IsDataOnlyBlueprint(AsBlueprint))
	{
		return MergeUtils::Merge(MergeArgs);
	}
	
	// Kludge to get the merge panel in the blueprint editor to show up:
	bool Success = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AsBlueprint);
	if (Success)
	{
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>( "Kismet" );

		FBlueprintEditor* BlueprintEditor = static_cast<FBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AsBlueprint, false));
		BlueprintEditor->CreateMergeToolTab();
	}

	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_Blueprint::Merge(const FAssetManualMergeArgs& MergeArgs) const
{
	UBlueprint* LocalBlueprint = Cast<UBlueprint>(MergeArgs.LocalAsset);
	if(!ensureMsgf(LocalBlueprint, TEXT("Merge LocalAsset is not a Blueprint")))
	{
		return Super::Merge(MergeArgs);
	}
	
	const UBlueprint* RemoteBlueprint = Cast<UBlueprint>(MergeArgs.RemoteAsset);
	if(!ensureMsgf(RemoteBlueprint, TEXT("Merge RemoteAsset is not a Blueprint")))
	{
		return Super::Merge(MergeArgs);
	}
	
	const UBlueprint* BaseBlueprint = Cast<UBlueprint>(MergeArgs.BaseAsset);
	if(!ensureMsgf(BaseBlueprint, TEXT("Merge BaseAsset is not a Blueprint")))
	{
		return Super::Merge(MergeArgs);
	}
	
	if ( // all assets are data only
		FBlueprintEditorUtils::IsDataOnlyBlueprint(LocalBlueprint) &&
		FBlueprintEditorUtils::IsDataOnlyBlueprint(RemoteBlueprint) &&
		FBlueprintEditorUtils::IsDataOnlyBlueprint(BaseBlueprint)
	)
	{
		return MergeUtils::Merge(MergeArgs);
	}
	check(MergeArgs.LocalAsset->GetClass() == MergeArgs.BaseAsset->GetClass());
	check(MergeArgs.LocalAsset->GetClass() == MergeArgs.RemoteAsset->GetClass());

	if (GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LocalBlueprint))
	{
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");

		FBlueprintEditor* BlueprintEditor = static_cast<FBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(LocalBlueprint, /*bFocusIfOpen =*/false));
		BlueprintEditor->CreateMergeToolTab(Cast<UBlueprint>(MergeArgs.BaseAsset), Cast<UBlueprint>(MergeArgs.RemoteAsset), 
			FOnMergeResolved::CreateLambda([ResolutionCallback = MergeArgs.ResolutionCallback](UPackage* MergedPackage, EMergeResult::Type Result)
			{
				FAssetMergeResults Results;
				Results.MergedPackage = MergedPackage;
				Results.Result = static_cast<EAssetMergeResult>(Result);
				ResolutionCallback.Execute(Results);
			})
		);		
	}
	
	 return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_Blueprint::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UBlueprint* OldBlueprint = Cast<UBlueprint>(DiffArgs.OldAsset);
	const UBlueprint* NewBlueprint = Cast<UBlueprint>(DiffArgs.NewAsset);

	if (NewBlueprint == nullptr && OldBlueprint == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}
	
	SBlueprintDiff::CreateDiffWindow(OldBlueprint, NewBlueprint, DiffArgs.OldRevision, DiffArgs.NewRevision, UBlueprint::StaticClass());
	return EAssetCommandResult::Handled;
}

UThumbnailInfo* UAssetDefinition_Blueprint::LoadThumbnailInfo(const FAssetData& InAssetData) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAssetData.GetAsset(), USceneThumbnailInfo::StaticClass());
}

EAssetCommandResult UAssetDefinition_Blueprint::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<FAssetData> OutAssetsThatFailedToLoad;
	for (UBlueprint* Blueprint : OpenArgs.LoadObjects<UBlueprint>({}, &OutAssetsThatFailedToLoad))
	{
		bool bLetOpen = true;
		if (!Blueprint->SkeletonGeneratedClass || !Blueprint->GeneratedClass)
		{
			bLetOpen = EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("FailedToLoadBlueprintWithContinue", "Blueprint could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed! Do you want to continue (it can crash the editor)?"));
		}
		if (bLetOpen)
		{
			FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
			TSharedRef< IBlueprintEditor > NewKismetEditor = BlueprintEditorModule.CreateBlueprintEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Blueprint, FBlueprintEditorUtils::ShouldOpenWithDataOnlyEditor(Blueprint));
		}
	}

	for (const FAssetData& UnableToLoadAsset : OutAssetsThatFailedToLoad)
	{
		FMessageDialog::Open( EAppMsgType::Ok,
			FText::Format(
				LOCTEXT("FailedToLoadBlueprint", "Blueprint '{0}' could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed!"),
				FText::FromName(UnableToLoadAsset.PackagePath)
			)
		);
	}

	return EAssetCommandResult::Handled;
}

TWeakPtr<IClassTypeActions> UAssetDefinition_Blueprint::GetClassTypeActions(const FAssetData& AssetData) const
{
	// Blueprints get the class type actions for their parent native class - this avoids us having to load the blueprint
	UClass* ParentClass = nullptr;
	FString ParentClassName;
	if (!AssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
	{
		AssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
	}
	
	if (!ParentClassName.IsEmpty())
	{
		UObject* Outer = nullptr;
		ResolveName(Outer, ParentClassName, false, false);
		ParentClass = FindObject<UClass>(Outer, *ParentClassName);
	}

	if (ParentClass)
	{
		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		return AssetToolsModule.Get().GetClassTypeActionsForClass(ParentClass);
	}

	return nullptr;
}

UFactory* UAssetDefinition_Blueprint::GetFactoryForBlueprintType(UBlueprint* InBlueprint) const
{
	UBlueprintFactory* BlueprintFactory = NewObject<UBlueprintFactory>();
	BlueprintFactory->ParentClass = InBlueprint->GeneratedClass;
	return BlueprintFactory;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_Blueprint
{
	static bool CanExecuteNewDerivedBlueprint(const FToolMenuContext& MenuContext, const FAssetData* SelectedBlueprintPtr)
	{
		const uint32 BPFlags = SelectedBlueprintPtr->GetTagValueRef<uint32>(FBlueprintTags::ClassFlags);
		if ((BPFlags & (CLASS_Deprecated)) != 0)
		{
			return false;
		}

		// Do not allow function libraries to have child classes created
		const FString& BPTypeStringView = SelectedBlueprintPtr->GetTagValueRef<FString>(FBlueprintTags::BlueprintType);
		const bool bIsFuncLib = (BPTypeStringView == TEXT("BPTYPE_FunctionLibrary")); 

		return !bIsFuncLib;
	}

	static void ExecuteNewDerivedBlueprint(const FToolMenuContext& MenuContext, const FAssetData* SelectedBlueprintPtr)
	{
		if (UBlueprint* ParentBlueprint = Cast<UBlueprint>(SelectedBlueprintPtr->GetAsset()))
		{
			UClass* TargetParentClass = ParentBlueprint->GeneratedClass;

			if (!FKismetEditorUtilities::CanCreateBlueprintOfClass(TargetParentClass))
			{
				FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("InvalidClassToMakeBlueprintFrom", "Invalid class with which to make a Blueprint."));
				return;
			}

			FString Name;
			FString PackageName;
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			AssetToolsModule.Get().CreateUniqueAssetName(ParentBlueprint->GetOutermost()->GetName(), TEXT("_Child"), PackageName, Name);
			const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

			const UAssetDefinition_Blueprint* BlueprintAssetDefinition = Cast<UAssetDefinition_Blueprint>(UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(ParentBlueprint->GetClass()));
			if (BlueprintAssetDefinition)
			{
				UFactory* Factory = BlueprintAssetDefinition->GetFactoryForBlueprintType(ParentBlueprint);

				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().CreateNewAsset(Name, PackagePath, ParentBlueprint->GetClass(), Factory);
			}
		}
	}

	static void ExecuteEditDefaults(const FToolMenuContext& MenuContext, TArray<FAssetData> BlueprintAssets)
	{
		TArray<UBlueprint*> Blueprints;

		FMessageLog EditorErrors("EditorErrors");
		EditorErrors.NewPage(LOCTEXT("ExecuteEditDefaultsNewLogPage", "Loading Blueprints"));

		for (const FAssetData& BlueprintAsset : BlueprintAssets)
		{
			if (UBlueprint* Object = Cast<UBlueprint>(BlueprintAsset.GetAsset()))
			{
				// If the blueprint is valid, allow it to be added to the list, otherwise log the error.
				if ( Object->SkeletonGeneratedClass && Object->GeneratedClass )
				{
					Blueprints.Add(Object);
				}
				else
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("ObjectName"), FText::FromString(Object->GetName()));
					EditorErrors.Error(FText::Format(LOCTEXT("LoadBlueprint_FailedLog", "{ObjectName} could not be loaded because it derives from an invalid class.  Check to make sure the parent class for this blueprint hasn't been removed!"), Arguments ) );
				}
			}
		}

		if ( Blueprints.Num() > 0 )
		{
			FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>( "Kismet" );
			TSharedRef< IBlueprintEditor > NewBlueprintEditor = BlueprintEditorModule.CreateBlueprintEditor(  EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), Blueprints );
		}

		// Report errors
		EditorErrors.Notify(LOCTEXT("OpenDefaults_Failed", "Opening Class Defaults Failed!"));
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UBlueprint::StaticClass());
	        
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
					{
						// TODO NDarnell re: EIncludeSubclasses::No, Temporary - Need to ensure we don't have duplicates for now, because no all subclasses of blueprint are of this class yet.
						if (const FAssetData* SelectedBlueprintPtr = Context->GetSingleSelectedAssetOfType(UBlueprint::StaticClass(), EIncludeSubclasses::No))
						{
							const TAttribute<FText> Label = LOCTEXT("Blueprint_NewDerivedBlueprint", "Create Child Blueprint Class");
							const TAttribute<FText> ToolTip = TAttribute<FText>::CreateLambda([SelectedBlueprintPtr]()
							{
								const uint32 BPFlags = SelectedBlueprintPtr->GetTagValueRef<uint32>(FBlueprintTags::ClassFlags);
								
								const FString BPTypeStringView = SelectedBlueprintPtr->GetTagValueRef<FString>(FBlueprintTags::BlueprintType);
								const bool bIsFuncLib = (BPTypeStringView == TEXT("BPTYPE_FunctionLibrary")); 
								
								if ((BPFlags & (CLASS_Deprecated)) != 0)
								{
									return LOCTEXT("Blueprint_NewDerivedBlueprintIsDeprecatedTooltip", "Blueprint class is deprecated, cannot derive a child Blueprint!");
								}
								else if (bIsFuncLib)
								{
									return LOCTEXT("Blueprint_NewDerivedBlueprintIsFunctionLibraryTooltip", "Cannot derive from Blueprint Function Libraries!");
								}
								else
								{
									return LOCTEXT("Blueprint_NewDerivedBlueprintTooltip", "Creates a Child Blueprint Class based on the current Blueprint, allowing you to create variants easily.");
									
								}
							});
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Blueprint");

							FToolUIAction DeriveNewBlueprint;
							DeriveNewBlueprint.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewDerivedBlueprint, SelectedBlueprintPtr);
							DeriveNewBlueprint.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteNewDerivedBlueprint, SelectedBlueprintPtr);
							InSection.AddMenuEntry("CreateChildBlueprintClass", Label, ToolTip, Icon, DeriveNewBlueprint);
						}

						TArray<FAssetData> SelectedBlueprints = Context->GetSelectedAssetsOfType(UBlueprint::StaticClass(), EIncludeSubclasses::No);
						if (SelectedBlueprints.Num() > 1)
						{
							TArray<UClass*> SelectedBlueprintParentClasses;
							Algo::Transform(SelectedBlueprints, SelectedBlueprintParentClasses, [](const FAssetData& BlueprintAsset){ return UBlueprint::GetBlueprintParentClassFromAssetTags(BlueprintAsset); });
							
							// Ensure that all the selected blueprints are actors
							const bool bAreAllSelectedBlueprintsActors =
								Algo::AllOf(SelectedBlueprintParentClasses, [](UClass* ParentClass){ return ParentClass && ParentClass->IsChildOf(AActor::StaticClass()); });
							
							if (bAreAllSelectedBlueprintsActors)
							{
								const TAttribute<FText> Label = LOCTEXT("Blueprint_EditDefaults", "Edit Shared Defaults");
								const TAttribute<FText> ToolTip = LOCTEXT("Blueprint_EditDefaultsTooltip", "Edit the shared default properties of the selected actor blueprints.");
								const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.BlueprintDefaults");
								const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteEditDefaults, MoveTemp(SelectedBlueprints));
								InSection.AddMenuEntry("Blueprint_EditDefaults", Label, ToolTip, Icon, UIAction);
							}
						}
					}
				}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
