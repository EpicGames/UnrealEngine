// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_BlueprintGeneratedClass.h"

#include "AssetDefinitionRegistry.h"
#include "ContentBrowserMenuContexts.h"
#include "Blueprint/BlueprintSupport.h"
#include "ContentBrowserModule.h"
#include "Factories/BlueprintFactory.h"
#include "IContentBrowserSingleton.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "IAssetTools.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_BlueprintGeneratedClass)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

TWeakPtr<IClassTypeActions> UAssetDefinition_BlueprintGeneratedClass::GetClassTypeActions(const FAssetData& AssetData) const
{
	// Blueprints get the class type actions for their parent native class.
	// Using asset tags avoids us having to load the blueprint
	UClass* ParentClass = nullptr;
	FString ParentClassName;
	if(!AssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName))
	{
		AssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
	}
	if (!ParentClassName.IsEmpty())
	{
		ParentClass = UClass::TryFindTypeSlow<UClass>(FPackageName::ExportTextPathToObjectPath(ParentClassName));
	}

	if (ParentClass)
	{
		return IAssetTools::Get().GetClassTypeActionsForClass(ParentClass);
	}

	return nullptr;
}

UClass* UAssetDefinition_BlueprintGeneratedClass::GetNewDerivedBlueprintClass() const
{
	return UBlueprint::StaticClass();
}

UFactory* UAssetDefinition_BlueprintGeneratedClass::GetFactoryForNewDerivedBlueprint(UBlueprintGeneratedClass* GeneratedClass) const
{
	UBlueprintFactory* BlueprintFactory = NewObject<UBlueprintFactory>();
	BlueprintFactory->ParentClass = GeneratedClass;
	return BlueprintFactory;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_BlueprintGeneratedClass
{
	static bool CanExecuteNewDerivedBlueprint(const FToolMenuContext& MenuContext, FAssetData GeneratedClassAsset)
	{
		const uint32 BPFlags = GeneratedClassAsset.GetTagValueRef<uint32>(FBlueprintTags::ClassFlags);
		if ((BPFlags & (CLASS_Deprecated)) == 0)
		{
			return true;
		}

		return false;
	}

	static void ExecuteNewDerivedBlueprint(const FToolMenuContext& MenuContext, FAssetData GeneratedClassAsset)
	{
		if (UBlueprintGeneratedClass* TargetParentClass = Cast<UBlueprintGeneratedClass>(GeneratedClassAsset.GetAsset()))
		{
			if (!FKismetEditorUtilities::CanCreateBlueprintOfClass(TargetParentClass))
			{
				FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("InvalidClassToMakeBlueprintFrom", "Invalid class with which to make a Blueprint."));
				return;
			}

			FString Name;
			FString PackageName;
			IAssetTools::Get().CreateUniqueAssetName(TargetParentClass->GetOutermost()->GetName(), TEXT("_Child"), PackageName, Name);
			const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

			const UAssetDefinition_BlueprintGeneratedClass* BPGC_AssetDefinition = Cast<UAssetDefinition_BlueprintGeneratedClass>(UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(TargetParentClass->GetClass()));
			if (BPGC_AssetDefinition)
			{
				UFactory* AssetFactory = BPGC_AssetDefinition->GetFactoryForNewDerivedBlueprint(TargetParentClass);
				UClass* AssetClass = BPGC_AssetDefinition->GetNewDerivedBlueprintClass();

				FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().CreateNewAsset(Name, PackagePath, AssetClass, AssetFactory);
			}
		}
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UBlueprintGeneratedClass::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					if (const FAssetData* SelectedBPGCPtr = CBContext->GetSingleSelectedAssetOfType(UBlueprintGeneratedClass::StaticClass()))
					{
						const UAssetDefinition_BlueprintGeneratedClass* BPGC_AssetDefinition = Cast<UAssetDefinition_BlueprintGeneratedClass>(UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(SelectedBPGCPtr->GetClass()));
						if (BPGC_AssetDefinition && BPGC_AssetDefinition->GetNewDerivedBlueprintClass() != nullptr)
						{
							FAssetData GeneratedClassAsset = *SelectedBPGCPtr;
							
							const TAttribute<FText> Label = LOCTEXT("BlueprintGeneratedClass_NewDerivedBlueprint", "Create Child Blueprint Class");
							const TAttribute<FText> ToolTip = TAttribute<FText>::CreateLambda([GeneratedClassAsset]()
							{
								const uint32 BPFlags = GeneratedClassAsset.GetTagValueRef<uint32>(FBlueprintTags::ClassFlags);
								const EBlueprintType BPType = static_cast<EBlueprintType>(GeneratedClassAsset.GetTagValueRef<int32>(FBlueprintTags::BlueprintType));
								
								if ((BPFlags & (CLASS_Deprecated)) != 0)
								{
									return LOCTEXT("Blueprint_NewDerivedBlueprintIsDeprecatedTooltip", "Blueprint class is deprecated, cannot derive a child Blueprint!");
								}
								else if (BPType == BPTYPE_FunctionLibrary)
								{
									return LOCTEXT("Blueprint_NewDerivedBlueprintIsFunctionLibraryTooltip", "Cannot derive from Blueprint Function Libraries!");
								}
								else
								{
									return LOCTEXT("Blueprint_NewDerivedBlueprintTooltip", "Creates a Child Blueprint Class based on the current Blueprint, allowing you to create variants easily.");
									
								}
							});
							const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Blueprint");

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewDerivedBlueprint, GeneratedClassAsset);
							UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteNewDerivedBlueprint, GeneratedClassAsset);
							InSection.AddMenuEntry("BlueprintGeneratedClass_NewDerivedBlueprint", Label, ToolTip, Icon, UIAction);
						}
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
