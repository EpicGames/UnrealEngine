// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlueprintGeneratedClass.h"
#include "Script/AssetDefinition_ClassTypeBase.h"

#include "AssetDefinition_BlueprintGeneratedClass.generated.h"

class UFactory;

UCLASS()
class UAssetDefinition_BlueprintGeneratedClass : public UAssetDefinition_ClassTypeBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_BlueprintGeneratedClass", "Compiled Blueprint Class"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(133, 173, 255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UBlueprintGeneratedClass::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { };
		return Categories;
	}
	// UAssetDefinition End

	// UAssetDefinition_ClassTypeBase Implementation
	virtual TWeakPtr<IClassTypeActions> GetClassTypeActions(const FAssetData& AssetData) const override;
	// End UAssetDefinition_ClassTypeBase Implementation

	virtual UClass* GetNewDerivedBlueprintClass() const;
	virtual UFactory* GetFactoryForNewDerivedBlueprint(UBlueprintGeneratedClass* GeneratedClass) const;
};
