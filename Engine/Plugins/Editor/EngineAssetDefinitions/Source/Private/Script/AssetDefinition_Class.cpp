// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_Class.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/FileManager.h"
#include "IAssetTools.h"
#include "Misc/Paths.h"
#include "SourceCodeNavigation.h"
#include "ContentBrowserMenuContexts.h"
#include "GameProjectGenerationModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Editor/UnrealEdEngine.h"
#include "ModuleDescriptor.h"
#include "Preferences/UnrealEdOptions.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "UnrealEdGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_Class)

#define LOCTEXT_NAMESPACE "AssetTypeActions"


UThumbnailInfo* UAssetDefinition_Class::LoadThumbnailInfo(const FAssetData& InAssetData) const
{
	// todo: jdale - CLASS - We need to generate and store proper thumbnail info for classes so that we can store their custom render transforms
	// This can't be stored in the UClass instance (like we do for Blueprints), so we'll need another place to store it
	// This will need to be accessible to FClassThumbnailScene::GetSceneThumbnailInfo
	return nullptr;
}

EAssetCommandResult UAssetDefinition_Class::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<FString> FilesToOpen;
	for (UClass* Class : OpenArgs.LoadObjects<UClass>())
	{
		FString ClassHeaderPath;
		if(FSourceCodeNavigation::FindClassHeaderPath(Class, ClassHeaderPath))
		{
			const FString AbsoluteHeaderPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ClassHeaderPath);
			FilesToOpen.Add(AbsoluteHeaderPath);
		}

		FString ClassSourcePath;
		if(FSourceCodeNavigation::FindClassSourcePath(Class, ClassSourcePath))
		{
			const FString AbsoluteSourcePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ClassSourcePath);
			FilesToOpen.Add(AbsoluteSourcePath);
		}
	}

	if (FilesToOpen.Num() > 0)
	{
		FSourceCodeNavigation::OpenSourceFiles(FilesToOpen);
	}

	return EAssetCommandResult::Handled;
}

TWeakPtr<IClassTypeActions> UAssetDefinition_Class::GetClassTypeActions(const FAssetData& AssetData) const
{
	UClass* Class = FindObject<UClass>(nullptr, *AssetData.GetObjectPathString());
	if (Class)
	{
		return IAssetTools::Get().GetClassTypeActionsForClass(Class);
	}

	return nullptr;
}


// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_Class
{
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UClass::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
				{
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.C++");

					auto IsCPPAllowed = []()
						{
							return ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed();
						};

					if (CBContext->SelectedAssets.Num() == 1)
					{
						if (UClass* const BaseClass = Cast<UClass>(CBContext->SelectedAssets[0].GetAsset()))
						{
							const FText BaseClassName = FText::FromName(BaseClass->GetFName());

							if (IsCPPAllowed())
							{
								// Only allow the New class option if we have a base class that we can actually derive from in one of our project modules
								FGameProjectGenerationModule& GameProjectGenerationModule = FGameProjectGenerationModule::Get();
								TArray<FModuleContextInfo> ProjectModules = GameProjectGenerationModule.GetCurrentProjectModules();
								if (GameProjectGenerationModule.IsValidBaseClassForCreation(BaseClass, ProjectModules))
								{
									const TAttribute<FText> Label = FText::Format(LOCTEXT("Class_NewDerivedCppClassLabel_CreateFrom", "Create C++ class derived from {0}"), BaseClassName);
									const TAttribute<FText> ToolTip = FText::Format(LOCTEXT("Class_NewDerivedCppClassTooltip_CreateFrom", "Create a new C++ class deriving from {0}."), BaseClassName);
									FToolUIAction UIAction;
									UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
										[BaseClass](const FToolMenuContext& InContext)
										{
											// Work out where the header file for the current class is, as we'll use that path as the default for the new class
											FString BaseClassPath;
											if (FSourceCodeNavigation::FindClassHeaderPath(BaseClass, BaseClassPath))
											{
												// Strip off the actual filename as we only need the path
												BaseClassPath = FPaths::GetPath(BaseClassPath);
											}

											FGameProjectGenerationModule::Get().OpenAddCodeToProjectDialog(
												FAddToProjectConfig()
												.ParentClass(BaseClass)
												.InitialPath(BaseClassPath)
												.ParentWindow(FGlobalTabmanager::Get()->GetRootWindow()));
										});
									InSection.AddMenuEntry("NewDerivedCppClass", Label, ToolTip, Icon, UIAction);
								}
							}

							if (FKismetEditorUtilities::CanCreateBlueprintOfClass(BaseClass))
							{
								const TAttribute<FText> Label = FText::Format(LOCTEXT("Class_NewDerivedBlueprintClassLabel_CreateFrom", "Create Blueprint class based on {0}"), BaseClassName);
								const TAttribute<FText> ToolTip = FText::Format(LOCTEXT("Class_NewDerivedBlueprintClassTooltip_CreateFrom", "Create a new Blueprint class based on {0}."), BaseClassName);
								FToolUIAction UIAction;
								UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
									[BaseClass](const FToolMenuContext& InContext)
									{
										FGameProjectGenerationModule::Get().OpenAddBlueprintToProjectDialog(
											FAddToProjectConfig()
											.ParentClass(BaseClass)
											.ParentWindow(FGlobalTabmanager::Get()->GetRootWindow()));
									});
								InSection.AddMenuEntry("NewDerivedBlueprintClass", Label, ToolTip, Icon, UIAction);
							}
						}
					}
					else
					{
						FToolUIAction CannotExecuteAction;
						CannotExecuteAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&) { return false; });

						if (IsCPPAllowed())
						{
							const TAttribute<FText> Label = LOCTEXT("Class_NewDerivedCppClassLabel_InvalidNumberOfBases", "New C++ class derived from...");
							const TAttribute<FText> ToolTip = LOCTEXT("Class_NewDerivedCppClassTooltip_InvalidNumberOfBases", "Can only create a derived C++ class when there is a single base class selected.");
							InSection.AddMenuEntry("NewDerivedCppClass", Label, ToolTip, Icon, CannotExecuteAction);
						}
						{
							const TAttribute<FText> Label = LOCTEXT("Class_NewDerivedBlueprintClassLabel_InvalidNumberOfBases", "New Blueprint class based on...");;
							const TAttribute<FText> ToolTip = LOCTEXT("Class_NewDerivedBlueprintClassTooltip_InvalidNumberOfBases", "Can only create a Blueprint class when there is a single base class selected.");
							InSection.AddMenuEntry("NewDerivedBlueprintClass", Label, ToolTip, Icon, CannotExecuteAction);
						}
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
