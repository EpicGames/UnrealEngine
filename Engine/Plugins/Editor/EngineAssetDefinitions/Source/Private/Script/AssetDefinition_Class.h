// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Script/AssetDefinition_ClassTypeBase.h"

#include "AssetDefinition_Class.generated.h"

struct FToolMenuContext;
struct FAssetData;
class UFactory;

UCLASS()
class UAssetDefinition_Class : public UAssetDefinition_ClassTypeBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Class", "C++ Class"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 255, 255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UClass::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Basic };
		return Categories;
	}
	
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End

	// UAssetDefinition_ClassTypeBase Implementation
	virtual TWeakPtr<IClassTypeActions> GetClassTypeActions(const FAssetData& AssetData) const override;
	// UAssetDefinition_ClassTypeBase End
};

