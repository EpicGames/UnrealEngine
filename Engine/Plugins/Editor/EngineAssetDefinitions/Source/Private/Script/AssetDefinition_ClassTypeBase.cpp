// Copyright Epic Games, Inc. All Rights Reserved.

#include "Script/AssetDefinition_ClassTypeBase.h"
#include "IClassTypeActions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ClassTypeBase)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

TSharedPtr<SWidget> UAssetDefinition_ClassTypeBase::GetThumbnailOverlay(const FAssetData& AssetData) const
{
	TSharedPtr<IClassTypeActions> ClassTypeActions = GetClassTypeActions(AssetData).Pin();
	if (ClassTypeActions.IsValid())
	{
		return ClassTypeActions->GetThumbnailOverlay(AssetData);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
