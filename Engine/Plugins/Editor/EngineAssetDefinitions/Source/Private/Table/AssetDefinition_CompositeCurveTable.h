// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/CompositeCurveTable.h"

#include "Table/AssetDefinition_CurveTable.h"
#include "AssetDefinition_CompositeCurveTable.generated.h"

UCLASS()
class UAssetDefinition_CompositeCurveTable : public UAssetDefinition_CurveTable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CompositeCurveTable", "Composite Curve Table"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCompositeCurveTable::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "CompositeCurveTable", "Data Table"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual bool CanImport() const override { return false; }
	// UAssetDefinition End
};
