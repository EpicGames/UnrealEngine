// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/CompositeDataTable.h"

#include "Table/AssetDefinition_DataTable.h"
#include "AssetDefinition_CompositeDataTable.generated.h"

UCLASS()
class UAssetDefinition_CompositeDataTable : public UAssetDefinition_DataTable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "FAssetTypeActions_CompositeDataTable", "Composite Data Table"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCompositeDataTable::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "CompositeDataTable", "Data Table"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual bool CanImport() const override { return false; }
	// UAssetDefinition End
};
