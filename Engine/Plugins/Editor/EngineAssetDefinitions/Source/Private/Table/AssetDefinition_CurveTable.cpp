// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_CurveTable.h"

#include "AssetDefinitionRegistry.h"
#include "IAssetTools.h"
#include "ToolMenus.h"
#include "IDesktopPlatform.h"
#include "Misc/FileHelper.h"
#include "EditorFramework/AssetImportData.h"
#include "Framework/Application/SlateApplication.h"
#include "AssetToolsModule.h"
#include "ContentBrowserMenuContexts.h"
#include "CurveTableEditorModule.h"
#include "DesktopPlatformModule.h"
#include "FindSourceFileInExplorer.h"
#include "Misc/Paths.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CurveTable)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

EAssetCommandResult UAssetDefinition_CurveTable::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UCurveTable* OldCurveTable = Cast<UCurveTable>(DiffArgs.OldAsset);
	const UCurveTable* NewCurveTable = Cast<UCurveTable>(DiffArgs.NewAsset);

	if (NewCurveTable == nullptr && OldCurveTable == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}

	const auto AssetNameFallback = [OldCurveTable, NewCurveTable]()
	{
		if (OldCurveTable)
		{
			return OldCurveTable->GetName();
		}
		if (NewCurveTable)
		{
			return NewCurveTable->GetName();
		}
		return FString();
	};

	// Build names for temp csv files
	const FString OldAssetName = OldCurveTable ? OldCurveTable->GetName() : AssetNameFallback();
	const FString NewAssetName = NewCurveTable ? NewCurveTable->GetName() : AssetNameFallback();
	
	const FString RelOldTempFileName = FString::Printf(TEXT("%sTemp%s-%s.csv"), *FPaths::DiffDir(), *OldAssetName, *DiffArgs.OldRevision.Revision);
	const FString AbsoluteOldTempFileName = FPaths::ConvertRelativePathToFull(RelOldTempFileName);
	
	const FString RelNewTempFileName = FString::Printf(TEXT("%sTemp%s-%s.csv"), *FPaths::DiffDir(), *NewAssetName, *DiffArgs.NewRevision.Revision);
	const FString AbsoluteNewTempFileName = FPaths::ConvertRelativePathToFull(RelNewTempFileName);

	// save temp files
	const FString OldCurveCSV = OldCurveTable? OldCurveTable->GetTableAsCSV() : TEXT("");
	const bool OldResult = FFileHelper::SaveStringToFile(OldCurveCSV, *AbsoluteOldTempFileName);
	const FString NewCurveCSV = NewCurveTable? NewCurveTable->GetTableAsCSV() : TEXT("");
	const bool NewResult = FFileHelper::SaveStringToFile(NewCurveCSV, *AbsoluteNewTempFileName);

	if (OldResult && NewResult)
	{
		const FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;

		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateDiffProcess(DiffCommand, AbsoluteOldTempFileName, AbsoluteNewTempFileName);

		return EAssetCommandResult::Handled;
	}

	return Super::PerformAssetDiff(DiffArgs);
}

EAssetCommandResult UAssetDefinition_CurveTable::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	FCurveTableEditorModule& CurveTableEditorModule = FModuleManager::LoadModuleChecked<FCurveTableEditorModule>( "CurveTableEditor" );
	
	for (UCurveTable* Table : OpenArgs.LoadObjects<UCurveTable>())
	{
		TSharedRef<ICurveTableEditor> NewCurveTableEditor = CurveTableEditorModule.CreateCurveTableEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Table);
	}
	
	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_CurveTable
{
	void ExecuteExportAsCSV(const FToolMenuContext& InContext)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		for (UCurveTable* CurveTable : Context->LoadSelectedObjects<UCurveTable>())
		{
			const FText Title = FText::Format(LOCTEXT("CurveTable_ExportCSVDialogTitle", "Export '{0}' as CSV..."), FText::FromString(*CurveTable->GetName()));
			const FString CurrentFilename = CurveTable->AssetImportData->GetFirstFilename();
			const FString FileTypes = TEXT("Data Table CSV (*.csv)|*.csv");

			TArray<FString> OutFilenames;
			DesktopPlatform->SaveFileDialog(
				ParentWindowWindowHandle,
				Title.ToString(),
				(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetPath(CurrentFilename),
				(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetBaseFilename(CurrentFilename) + TEXT(".csv"),
				FileTypes,
				EFileDialogFlags::None,
				OutFilenames
				);

			if (OutFilenames.Num() > 0)
			{
				FFileHelper::SaveStringToFile(CurveTable->GetTableAsCSV(), *OutFilenames[0]);
			}
		}
	}

	void ExecuteExportAsJSON(const FToolMenuContext& InContext)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		for (UCurveTable* CurveTable : Context->LoadSelectedObjects<UCurveTable>())
		{
			const FText Title = FText::Format(LOCTEXT("DataTable_ExportJSONDialogTitle", "Export '{0}' as JSON..."), FText::FromString(*CurveTable->GetName()));
			const FString CurrentFilename = CurveTable->AssetImportData->GetFirstFilename();
			const FString FileTypes = TEXT("Data Table JSON (*.json)|*.json");

			TArray<FString> OutFilenames;
			DesktopPlatform->SaveFileDialog(
				ParentWindowWindowHandle,
				Title.ToString(),
				(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetPath(CurrentFilename),
				(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetBaseFilename(CurrentFilename) + TEXT(".json"),
				FileTypes,
				EFileDialogFlags::None,
				OutFilenames
			);

			if (OutFilenames.Num() > 0)
			{
				FFileHelper::SaveStringToFile(CurveTable->GetTableAsJSON(), *OutFilenames[0]);
			}
		}
	}

	void ExecuteFindSourceFileInExplorer(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		
		TArray<FString> ImportPaths;

		for (const FAssetData& Asset : Context->SelectedAssets)
		{
			FAssetSourceFilesArgs GetSourceFilesArgs;
			GetSourceFilesArgs.Assets = TConstArrayView<FAssetData>(&Asset, 1);
			if (const UAssetDefinition* AssetDefination = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(Asset))
			{ 
				AssetDefination->GetSourceFiles(GetSourceFilesArgs, [&ImportPaths](const FAssetSourceFilesResult& AssetImportInfo)
				{
					ImportPaths.Add(AssetImportInfo.FilePath);
					return true;
				});
			}
		}

		TArray<FString> PotentialFileExtensions;
		PotentialFileExtensions.Add(TEXT(".xls"));
		PotentialFileExtensions.Add(TEXT(".xlsm"));
		PotentialFileExtensions.Add(TEXT(".csv"));
		PotentialFileExtensions.Add(TEXT(".json"));

		UE::AssetTools::ExecuteFindSourceFileInExplorer(ImportPaths, PotentialFileExtensions);
	}

	bool CanExecuteFindSourceFileInExplorer(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		TArray<FString> ImportPaths;
		for (const FAssetData& Asset : Context->SelectedAssets)
		{
			FAssetSourceFilesArgs GetSourceFilesArgs;
			GetSourceFilesArgs.Assets = TConstArrayView<FAssetData>(&Asset, 1);
			if (const UAssetDefinition* AssetDefination = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(Asset))
			{
				AssetDefination->GetSourceFiles(GetSourceFilesArgs, [&ImportPaths](const FAssetSourceFilesResult& AssetImportInfo)
				{
					ImportPaths.Add(AssetImportInfo.FilePath);
					return true;
				});
			}
		}

		TArray<FString> PotentialFileExtensions;
		PotentialFileExtensions.Add(TEXT(".xls"));
		PotentialFileExtensions.Add(TEXT(".xlsm"));
		PotentialFileExtensions.Add(TEXT(".csv"));
		PotentialFileExtensions.Add(TEXT(".json"));

		return UE::AssetTools::CanExecuteFindSourceFileInExplorer(ImportPaths, PotentialFileExtensions);
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UCurveTable::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("CurveTable_ExportAsCSV", "Export as CSV");
					const TAttribute<FText> ToolTip = LOCTEXT("CurveTable_ExportAsCSVTooltip", "Export the curve table as a file containing CSV data.");
					const FSlateIcon Icon = FSlateIcon();

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteExportAsCSV);
					InSection.AddMenuEntry("CurveTable_ExportAsCSV", Label, ToolTip, Icon, UIAction);
				}
				
				{
					const TAttribute<FText> Label = LOCTEXT("CurveTable_ExportAsJSON", "Export as JSON");
					const TAttribute<FText> ToolTip = LOCTEXT("CurveTable_ExportAsJSONTooltip", "Export the curve table as a file containing JSON data.");
					const FSlateIcon Icon = FSlateIcon();

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteExportAsJSON);
					InSection.AddMenuEntry("CurveTable_ExportAsJSON", Label, ToolTip, Icon, UIAction);
				}

				{
					const TAttribute<FText> Label = LOCTEXT("CurveTable_OpenSourceData", "Open Source Data");
					const TAttribute<FText> ToolTip = LOCTEXT("CurveTable_OpenSourceDataTooltip", "Opens the curve table's source data file in an external editor. It will search using the following extensions: .xls/.xlsm/.csv/.json");
					const FSlateIcon Icon = FSlateIcon();

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(&ExecuteFindSourceFileInExplorer);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteFindSourceFileInExplorer);
					InSection.AddMenuEntry("CurveTable_OpenSourceData", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
