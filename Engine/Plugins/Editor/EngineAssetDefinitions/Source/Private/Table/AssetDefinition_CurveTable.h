// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/CurveTable.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_CurveTable.generated.h"

UCLASS()
class UAssetDefinition_CurveTable : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CurveTable", "Curve Table"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(62, 140, 35)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCurveTable::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "AssetDefinition_CurveTableSubMenu", "Curve"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	virtual bool CanImport() const override { return true; }
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
