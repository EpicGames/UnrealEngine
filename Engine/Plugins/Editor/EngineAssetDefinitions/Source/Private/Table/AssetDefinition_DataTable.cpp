// Copyright Epic Games, Inc. All Rights Reserved.

#include "Table/AssetDefinition_DataTable.h"

#include "AssetDefinitionRegistry.h"
#include "IAssetTools.h"
#include "ToolMenus.h"
#include "IDesktopPlatform.h"
#include "Misc/FileHelper.h"
#include "EditorFramework/AssetImportData.h"
#include "Misc/MessageDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/Paths.h"
#include "Settings/EditorLoadingSavingSettings.h"

#include "DataTableEditorModule.h"
#include "DesktopPlatformModule.h"
#include "AssetToolsModule.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ContentBrowserMenuContexts.h"
#include "FindSourceFileInExplorer.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_DataTable)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

static const FName NAME_RowStructure(TEXT("RowStructure"));

FText UAssetDefinition_DataTable::GetAssetDisplayName(const FAssetData& AssetData) const
{
	if (AssetData.IsValid())
	{
		const FAssetDataTagMapSharedView::FFindTagResult RowStructureTag = AssetData.TagsAndValues.FindTag(NAME_RowStructure);
		if (RowStructureTag.IsSet())
		{
			// Handle full path names and deprecated short class names
			const FTopLevelAssetPath ClassPath = FAssetData::TryConvertShortClassNameToPathName(*RowStructureTag.GetValue(), ELogVerbosity::Log);

			if (const UScriptStruct* FoundStruct = UClass::TryFindTypeSlow<UScriptStruct>(ClassPath.ToString(), EFindFirstObjectOptions::ExactClass))
			{
				return FText::Format(LOCTEXT("DataTableWithRowType", "Data Table ({0})"), FoundStruct->GetDisplayNameText());
			}
		}
	}

	return FText::GetEmpty();
}

// Attempts to export temporary CSV files and diff those. If that fails we fall back to diffing the data table assets directly.
EAssetCommandResult UAssetDefinition_DataTable::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	const UDataTable* OldDataTable = Cast<UDataTable>(DiffArgs.OldAsset);
	const UDataTable* NewDataTable = Cast<UDataTable>(DiffArgs.NewAsset);

	if (OldDataTable == nullptr && NewDataTable == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}
	
	// Build names for temp csv files
	const auto AssetNameFallback = [NewDataTable, OldDataTable]()
	{
		if (OldDataTable)
		{
			return OldDataTable->GetName();
		}
		if (NewDataTable)
		{
			return NewDataTable->GetName();
		}
		return FString();
	};

	const FString OldAssetName = OldDataTable ? OldDataTable->GetName() : AssetNameFallback();
	const FString NewAssetName = NewDataTable ? NewDataTable->GetName() : AssetNameFallback();
	
	const FString RelOldTempFileName = FString::Printf(TEXT("%sTemp%s-%s.csv"), *FPaths::DiffDir(), *OldAssetName, *DiffArgs.OldRevision.Revision);
	const FString AbsoluteOldTempFileName = FPaths::ConvertRelativePathToFull(RelOldTempFileName);
	
	const FString RelNewTempFileName = FString::Printf(TEXT("%sTemp%s-%s.csv"), *FPaths::DiffDir(), *NewAssetName, *DiffArgs.NewRevision.Revision);
	const FString AbsoluteNewTempFileName = FPaths::ConvertRelativePathToFull(RelNewTempFileName);

	// save temp files
	const FString OldTableCSV = OldDataTable? OldDataTable->GetTableAsCSV() : TEXT("");
	const bool OldResult = FFileHelper::SaveStringToFile(OldTableCSV, *AbsoluteOldTempFileName);
	const FString NewTableCSV = NewDataTable? NewDataTable->GetTableAsCSV() : TEXT("");
	const bool NewResult = FFileHelper::SaveStringToFile(NewTableCSV, *AbsoluteNewTempFileName);

	if (OldResult && NewResult)
	{
		const FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;

		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateDiffProcess(DiffCommand, AbsoluteOldTempFileName, AbsoluteNewTempFileName);

		return EAssetCommandResult::Handled;
	}

	return Super::PerformAssetDiff(DiffArgs);
}

EAssetCommandResult UAssetDefinition_DataTable::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UDataTable*> DataTablesToOpen;
	TArray<UDataTable*> InvalidDataTables;

	for (UDataTable* Table : OpenArgs.LoadObjects<UDataTable>())
	{
		if (Table->GetRowStruct())
		{
			DataTablesToOpen.Add(Table);
		}
		else
		{
			InvalidDataTables.Add(Table);
		}
	}

	if (InvalidDataTables.Num() > 0)
	{
		FTextBuilder DataTablesListText;
		DataTablesListText.Indent();
		for (UDataTable* Table : InvalidDataTables)
		{
			const FTopLevelAssetPath ResolvedRowStructName = Table->GetRowStructPathName();
			DataTablesListText.AppendLineFormat(LOCTEXT("DataTable_MissingRowStructListEntry", "* {0} (Row Structure: {1})"), FText::FromString(Table->GetName()), FText::FromString(ResolvedRowStructName.ToString()));
		}

		const EAppReturnType::Type DlgResult = FMessageDialog::Open(
			EAppMsgType::YesNoCancel, 
			FText::Format(LOCTEXT("DataTable_MissingRowStructMsg", "The following Data Tables are missing their row structure and will not be editable.\n\n{0}\n\nDo you want to open these data tables?"), DataTablesListText.ToText()), 
			LOCTEXT("DataTable_MissingRowStructTitle", "Continue?")
			);

		switch(DlgResult)
		{
		case EAppReturnType::Yes:
			DataTablesToOpen.Append(InvalidDataTables);
			break;
		case EAppReturnType::Cancel:
			return EAssetCommandResult::Handled;
		default:
			break;
		}
	}

	FDataTableEditorModule& DataTableEditorModule = FModuleManager::LoadModuleChecked<FDataTableEditorModule>("DataTableEditor");
	for (UDataTable* Table : DataTablesToOpen)
	{
		DataTableEditorModule.CreateDataTableEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Table);
	}

	return EAssetCommandResult::Handled;
}


// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_DataTable
{
	void ExecuteExportAsCSV(const FToolMenuContext& InContext)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		for (UDataTable* DataTable : Context->LoadSelectedObjects<UDataTable>())
		{
			const FText Title = FText::Format(LOCTEXT("DataTable_ExportCSVDialogTitle", "Export '{0}' as CSV..."), FText::FromString(*DataTable->GetName()));
			const FString CurrentFilename = DataTable->AssetImportData->GetFirstFilename();
			const FString FileTypes = TEXT("Data Table CSV (*.csv)|*.csv");

			TArray<FString> OutFilenames;
			DesktopPlatform->SaveFileDialog(
				ParentWindowWindowHandle,
				Title.ToString(),
				(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetPath(CurrentFilename),
				(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetBaseFilename(CurrentFilename) + TEXT(".csv"),
				FileTypes,
				EFileDialogFlags::None,
				OutFilenames
				);

			if (OutFilenames.Num() > 0)
			{
				FFileHelper::SaveStringToFile(DataTable->GetTableAsCSV(), *OutFilenames[0]);
			}
		}
	}

	void ExecuteExportAsJSON(const FToolMenuContext& InContext)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		for (UDataTable* DataTable : Context->LoadSelectedObjects<UDataTable>())
		{
			const FText Title = FText::Format(LOCTEXT("DataTable_ExportJSONDialogTitle", "Export '{0}' as JSON..."), FText::FromString(*DataTable->GetName()));
			const FString CurrentFilename = DataTable->AssetImportData->GetFirstFilename();
			const FString FileTypes = TEXT("Data Table JSON (*.json)|*.json");

			TArray<FString> OutFilenames;
			DesktopPlatform->SaveFileDialog(
				ParentWindowWindowHandle,
				Title.ToString(),
				(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetPath(CurrentFilename),
				(CurrentFilename.IsEmpty()) ? TEXT("") : FPaths::GetBaseFilename(CurrentFilename) + TEXT(".json"),
				FileTypes,
				EFileDialogFlags::None,
				OutFilenames
			);

			if (OutFilenames.Num() > 0)
			{
				FFileHelper::SaveStringToFile(DataTable->GetTableAsJSON(EDataTableExportFlags::UseJsonObjectsForStructs), *OutFilenames[0]);
			}
		}
	}

	void ExecuteFindSourceFileInExplorer(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		
		TArray<FString> ImportPaths;
		for (const FAssetData& Asset : Context->SelectedAssets)
		{
			FAssetSourceFilesArgs GetSourceFilesArgs;
			GetSourceFilesArgs.Assets = TConstArrayView<FAssetData>(&Asset, 1);
			if (const UAssetDefinition* AssetDefination = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(Asset))
			{ 
				AssetDefination->GetSourceFiles(GetSourceFilesArgs, [&ImportPaths](const FAssetSourceFilesResult& AssetImportInfo)
				{
					ImportPaths.Add(AssetImportInfo.FilePath);
					return true;
				});
			}
		}

		TArray<FString> PotentialFileExtensions;
		PotentialFileExtensions.Add(TEXT(".xls"));
		PotentialFileExtensions.Add(TEXT(".xlsm"));
		PotentialFileExtensions.Add(TEXT(".csv"));
		PotentialFileExtensions.Add(TEXT(".json"));

		UE::AssetTools::ExecuteFindSourceFileInExplorer(ImportPaths, PotentialFileExtensions);
	}

	bool CanExecuteFindSourceFileInExplorer(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		TArray<FString> ImportPaths;
		for (const FAssetData& Asset : Context->SelectedAssets)
		{
			FAssetSourceFilesArgs GetSourceFilesArgs;
			GetSourceFilesArgs.Assets = TConstArrayView<FAssetData>(&Asset, 1);
			if (const UAssetDefinition* AssetDefination = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(Asset))
			{ 
				AssetDefination->GetSourceFiles(GetSourceFilesArgs, [&ImportPaths](const FAssetSourceFilesResult& AssetImportInfo)
				{
					ImportPaths.Add(AssetImportInfo.FilePath);
					return true;
				});
			}
		}

		TArray<FString> PotentialFileExtensions;
		PotentialFileExtensions.Add(TEXT(".xls"));
		PotentialFileExtensions.Add(TEXT(".xlsm"));
		PotentialFileExtensions.Add(TEXT(".csv"));
		PotentialFileExtensions.Add(TEXT(".json"));

		return UE::AssetTools::CanExecuteFindSourceFileInExplorer(ImportPaths, PotentialFileExtensions);
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UDataTable::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("DataTable_ExportAsCSV", "Export as CSV");
					const TAttribute<FText> ToolTip = LOCTEXT("DataTable_ExportAsCSVTooltip", "Export the data table as a file containing CSV data.");
					const FSlateIcon Icon = FSlateIcon();

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteExportAsCSV);
					InSection.AddMenuEntry("DataTable_ExportAsCSV", Label, ToolTip, Icon, UIAction);
				}
				
				{
					const TAttribute<FText> Label = LOCTEXT("DataTable_ExportAsJSON", "Export as JSON");
					const TAttribute<FText> ToolTip = LOCTEXT("DataTable_ExportAsJSONTooltip", "Export the data table as a file containing JSON data.");
					const FSlateIcon Icon = FSlateIcon();

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteExportAsJSON);
					InSection.AddMenuEntry("DataTable_ExportAsJSON", Label, ToolTip, Icon, UIAction);
				}
				{
					const TAttribute<FText> Label = LOCTEXT("DataTable_OpenSourceData", "Open Source Data");
					const TAttribute<FText> ToolTip = LOCTEXT("DataTable_OpenSourceDataTooltip", "Opens the data table's source data file in an external editor. It will search using the following extensions: .xls/.xlsm/.csv/.json");
					const FSlateIcon Icon = FSlateIcon();

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(&ExecuteFindSourceFileInExplorer);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteFindSourceFileInExplorer);
					InSection.AddMenuEntry("DataTable_OpenSourceData", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

/*
void FAssetTypeActions_DataTable::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	auto Tables = GetTypedWeakObjectPtrs<UObject>(InObjects);
	
	TArray<FString> ImportPaths;
	for (auto TableIter = Tables.CreateConstIterator(); TableIter; ++TableIter)
	{
		const UDataTable* CurTable = Cast<UDataTable>((*TableIter).Get());
		if (CurTable)
		{
			CurTable->AssetImportData->ExtractFilenames(ImportPaths);
		}
	}

	UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(YourAsset)->GetSourceFiles()

	TArray<FString> PotentialFileExtensions;
	PotentialFileExtensions.Add(TEXT(".xls"));
	PotentialFileExtensions.Add(TEXT(".xlsm"));
	PotentialFileExtensions.Add(TEXT(".csv"));
	PotentialFileExtensions.Add(TEXT(".json"));
	Section.AddMenuEntry(
		"DataTable_OpenSourceData",
		LOCTEXT("DataTable_OpenSourceData", "Open Source Data"),
		LOCTEXT("DataTable_OpenSourceDataTooltip", "Opens the data table's source data file in an external editor. It will search using the following extensions: .xls/.xlsm/.csv/.json"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP( this, &ExecuteFindSourceFileInExplorer, ImportPaths, PotentialFileExtensions ),
			FCanExecuteAction::CreateSP(this, &CanExecuteFindSourceFileInExplorer, ImportPaths, PotentialFileExtensions)
			)
		); 
}
*/

#undef LOCTEXT_NAMESPACE
