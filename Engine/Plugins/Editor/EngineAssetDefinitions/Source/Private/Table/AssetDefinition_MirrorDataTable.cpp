// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_MirrorDataTable.h"

#include "AssetDefinitionRegistry.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_MirrorDataTable)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_MirrorDataTable
{
	void ExecuteFindReplace(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		FScopedTransaction Transaction(LOCTEXT("MirrorDataTable_FindReplace", "Update using Find Replace"));
		for (UMirrorDataTable* MirrorDataTable : CBContext->LoadSelectedObjects<UMirrorDataTable>())
		{
			MirrorDataTable->UpdateFromFindReplaceExpressions(UMirrorDataTable::FFindReplaceOptions::UpdateExisting().WithNotification()); 
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, [] {
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
			{
				FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UMirrorDataTable::StaticClass());

				FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection);
						{
							const TAttribute<FText> Label = LOCTEXT("MirrorDataTable_FindReplace", "Update using Find Replace");
							const TAttribute<FText> ToolTip = LOCTEXT("MirrorDataTable_FindReplaceTooltip", "Updates existing mirror pairs and disables stale ones. Missing entries are not added.");
							const FSlateIcon Icon = FSlateIcon();

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteFindReplace);
							InSection.AddMenuEntry("MirrorDataTable_FindReplace", Label, ToolTip, Icon, UIAction);
						}
					}));
			}));
		});
}

#undef LOCTEXT_NAMESPACE
