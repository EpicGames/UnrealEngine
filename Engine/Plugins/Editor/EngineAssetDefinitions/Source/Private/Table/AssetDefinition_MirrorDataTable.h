// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/MirrorDataTable.h"

#include "Table/AssetDefinition_DataTable.h"
#include "AssetDefinition_MirrorDataTable.generated.h"

UCLASS()
class UAssetDefinition_MirrorDataTable : public UAssetDefinition_DataTable
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("UMirrorDataTable", "FAssetTypeActions_MirrorDataTable", "Mirror Data Table"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UMirrorDataTable::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Animation };
		return Categories;
	}
	virtual bool CanImport() const override { return false; }
	// UAssetDefinition End
};
