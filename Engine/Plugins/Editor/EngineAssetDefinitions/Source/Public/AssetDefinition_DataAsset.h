// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Engine/DataAsset.h"

#include "AssetDefinition_DataAsset.generated.h"

#define UE_API ENGINEASSETDEFINITIONS_API

UCLASS(MinimalAPI)
class UAssetDefinition_DataAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Implementation
	UAssetDefinition_DataAsset()
	{
		IncludeClassInFilter = EIncludeClassInFilter::Always;
	}
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "DataAsset", "Data Asset"); }
	UE_API virtual FText GetAssetDisplayName(const FAssetData& AssetData) const override;
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(201, 29, 85)); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "DataAsset_SubMenuData", "Data Asset"), ECategoryMenuType::Section)
			};
		return Categories;
	}

	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UDataAsset::StaticClass(); }
	
	UE_API virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;

	UE_API virtual bool CanMerge() const override;
	UE_API virtual EAssetCommandResult Merge(const FAssetAutomaticMergeArgs& MergeArgs) const override;
	UE_API virtual EAssetCommandResult Merge(const FAssetManualMergeArgs& MergeArgs) const override;
};

#undef UE_API
