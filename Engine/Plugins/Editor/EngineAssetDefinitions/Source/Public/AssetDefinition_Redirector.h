// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_Redirector.generated.h"

#define UE_API ENGINEASSETDEFINITIONS_API

enum class EAssetCommandResult : uint8;
struct FAssetActivateArgs;

UCLASS(MinimalAPI)
class UAssetDefinition_Redirector : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "Redirector", "Redirector"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128, 128, 128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UObjectRedirector::StaticClass(); }
	UE_API virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
};

#undef UE_API
