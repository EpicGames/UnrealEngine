// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "AssetDefinitionDefault.h"

#include "Styling/AppStyle.h"
#include "AssetDefinition_World.generated.h"

#define UE_API ENGINEASSETDEFINITIONS_API

enum class EAssetCommandResult : uint8;
struct FAssetActivateArgs;
struct FAssetCategoryPath;
struct FAssetOpenArgs;
struct FAssetSupportResponse;

UCLASS(MinimalAPI)
class UAssetDefinition_World : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_World", "Level"); }
	virtual FLinearColor GetAssetColor() const override { return FAppStyle::Get().GetColor("LevelEditor.AssetColor"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UWorld::StaticClass(); }
	UE_API virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;

	UE_API virtual TArray<FAssetData> PrepareToActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	UE_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	UE_API virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	
	UE_API virtual FAssetSupportResponse CanRename(const FAssetData& InAsset) const override;
	UE_API virtual FAssetSupportResponse CanDuplicate(const FAssetData& InAsset) const override;

	UE_API virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAsset) const override;
	// UAssetDefinition End
	
public:
	UE_API bool IsPartitionWorldInUse(const FAssetData& InAsset) const;
};

#undef UE_API
