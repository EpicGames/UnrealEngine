// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "Script/AssetDefinition_ClassTypeBase.h"
#include "AssetDefinition_Blueprint.generated.h"

struct FAssetFilterData;

struct FToolMenuContext;
struct FAssetData;
class UFactory;

UCLASS(MinimalAPI)
class UAssetDefinition_Blueprint : public UAssetDefinition_ClassTypeBase
{
	GENERATED_BODY()

public:
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_Blueprint", "Blueprint Class"); }
	ENGINEASSETDEFINITIONS_API virtual FText GetAssetDisplayName(const FAssetData& AssetData) const override;
	ENGINEASSETDEFINITIONS_API virtual FText GetAssetDescription(const FAssetData& AssetData) const override;
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor( 63, 126, 255 )); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UBlueprint::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Basic, EAssetCategoryPaths::Blueprint };
		return Categories;
	}

	virtual bool CanMerge() const override { return true; }
	ENGINEASSETDEFINITIONS_API virtual EAssetCommandResult Merge(const FAssetAutomaticMergeArgs& MergeArgs) const override;
	ENGINEASSETDEFINITIONS_API virtual EAssetCommandResult Merge(const FAssetManualMergeArgs& MergeArgs) const override;
	
	ENGINEASSETDEFINITIONS_API virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;

	ENGINEASSETDEFINITIONS_API virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;
	ENGINEASSETDEFINITIONS_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End

	// UAssetDefinition_ClassTypeBase Implementation
	ENGINEASSETDEFINITIONS_API virtual TWeakPtr<IClassTypeActions> GetClassTypeActions(const FAssetData& AssetData) const override;
	// UAssetDefinition_ClassTypeBase End
	
public:
	ENGINEASSETDEFINITIONS_API virtual UFactory* GetFactoryForBlueprintType(UBlueprint* InBlueprint) const;
};
