// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ClassTypeBase.generated.h"

class IClassTypeActions;

/** Base class for "class type" assets (C++ classes and Blueprints */
UCLASS(Abstract, MinimalAPI)
class UAssetDefinition_ClassTypeBase : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const override
	{
		return FAssetSupportResponse::NotSupported();
	}
	ENGINEASSETDEFINITIONS_API virtual TSharedPtr<class SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override;
	// UAssetDefinition End

	/** Get the class type actions for this asset */
	virtual TWeakPtr<IClassTypeActions> GetClassTypeActions(const FAssetData& AssetData) const
	{
		return TWeakPtr<IClassTypeActions>();
	}
};
