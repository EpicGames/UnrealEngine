// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_DataTable.generated.h"

#define UE_API ENGINEASSETDEFINITIONS_API

UCLASS(MinimalAPI)
class UAssetDefinition_DataTable : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_DataTable", "Data Table"); }
	UE_API virtual FText GetAssetDisplayName(const FAssetData& AssetData) const override;
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(62, 140, 35)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UDataTable::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Data, NSLOCTEXT("AssetDefinition", "DataTable", "Data Table"), ECategoryMenuType::Section) };
		return Categories;
	}
	virtual bool CanImport() const override { return true; }
	UE_API virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	UE_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};

#undef UE_API
